// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HAL/Platform.h"

class IPlugin;

namespace UE::PluginManager::Private
{
	/// @brief Interface to get around dependency problems between Projects and CoreUObject
	/// interface allows CoreUObject to run login in PluginManager that Projects can't run
	class ICoreUObjectPluginManager
	{
	public:
		virtual void OnPluginLoad(IPlugin& Plugin) = 0;
		virtual void OnPluginUnload(IPlugin& Plugin) = 0;

		/**
		* API for deferring calls to CollectGarbage (and leak detection).
		*/
		virtual void SuppressPluginUnloadGC() = 0;
		virtual void ResumePluginUnloadGC() = 0;
	};

	PROJECTS_API void SetCoreUObjectPluginManager(ICoreUObjectPluginManager& Handler);
}