// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomBuildSteps.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"
#include "JsonUtils/JsonConversion.h"
#include "RapidJsonPluginLoading.h"

#define LOCTEXT_NAMESPACE "ModuleDescriptor"

bool FCustomBuildSteps::IsEmpty() const
{
	return HostPlatformToCommands.Num() == 0;
}

FCustomBuildSteps UE::Projects::Private::ReadCustomBuildSteps(UE::Json::FConstObject Object, const TCHAR* FieldName)
{
	FCustomBuildSteps Result{};

	TOptional<Json::FConstObject> StepsObject = UE::Json::GetObjectField(Object, FieldName);
	if (StepsObject.IsSet())
	{
		for (const Json::FMember& Member : *StepsObject)
		{
			TArray<FString>& Commands = Result.HostPlatformToCommands.FindOrAdd(Member.name.GetString());
			if (Member.value.IsArray())
			{
				Json::FConstArray CommandsArray = Member.value.GetArray();
				Commands.Reserve(CommandsArray.Size());
				for (const Json::FValue& CommandValue : CommandsArray)
				{
					if (CommandValue.IsString())
					{
						Commands.Emplace(CommandValue.GetString());
					}
				}
			}
		}
	}

	return Result;
}

void FCustomBuildSteps::Read(const FJsonObject& Object, const FString& FieldName)
{
	UE::Projects::Private::ReadFromDefaultJsonHelper(
		Object,
		[&FieldName, this](UE::Json::FConstObject RootObject)
		{
			*this = UE::Projects::Private::ReadCustomBuildSteps(RootObject, *FieldName);
			return true;
		}
	);
}


void FCustomBuildSteps::Write(TJsonWriter<>& Writer, const FString& FieldName) const
{
	if (!IsEmpty())
	{
		TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		UpdateJson(*JsonObject, FieldName);

		if (TSharedPtr<FJsonValue> StepsValue = JsonObject->TryGetField(FieldName))
		{
			FJsonSerializer::Serialize(StepsValue, FieldName, Writer);
		}
	}
}

void FCustomBuildSteps::UpdateJson(FJsonObject& JsonObject, const FString& FieldName) const
{
	if (!IsEmpty())
	{
		TSharedPtr<FJsonObject> StepsObject;
		{
			const TSharedPtr<FJsonObject>* StepsObjectPtr = nullptr;
			if (JsonObject.TryGetObjectField(FieldName, StepsObjectPtr) && StepsObjectPtr)
			{
				StepsObject = *StepsObjectPtr;
			}
			else
			{
				StepsObject = MakeShared<FJsonObject>();
				JsonObject.SetObjectField(FieldName, StepsObject);
			}
		}

		if (ensure(StepsObject.IsValid()))
		{
			for (const TPair<FString, TArray<FString>>& HostPlatformAndCommands : HostPlatformToCommands)
			{
				TArray<TSharedPtr<FJsonValue>> CommandValues;

				for (const FString& Command : HostPlatformAndCommands.Value)
				{
					CommandValues.Add(MakeShareable(new FJsonValueString(Command)));
				}

				StepsObject->SetArrayField(HostPlatformAndCommands.Key, CommandValues);
			}
		}
	}
	else
	{
		JsonObject.RemoveField(FieldName);
	}
}

#undef LOCTEXT_NAMESPACE

