// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/JsonSerializer.h"

namespace JsonExtensions
{
	/** Get the field named FieldName as an array of strings. Returns false if it doesn't exist or any member cannot be converted. */
	inline bool TryGetStringArrayFieldWithDeprecatedFallback(const FJsonObject& JsonObject, const FString& FieldName, const FString& DeprecatedFieldName, TArray<FString>& OutArray)
	{
		if (JsonObject.TryGetStringArrayField(FieldName, /*out*/ OutArray))
		{
			return true;
		}
		else if (JsonObject.TryGetStringArrayField(DeprecatedFieldName, /*out*/ OutArray))
		{
			//@TODO: Warn about deprecated field fallback?
			return true;
		}
		else
		{
			return false;
		}
	}

	/** Get the field named FieldName as an array of enums. Returns false if it doesn't exist or any member is not a string. */
	template<typename TEnum>
	inline bool TryGetEnumArrayFieldWithDeprecatedFallback(const FJsonObject& JsonObject, const FString& FieldName, const FString& DeprecatedFieldName, TArray<TEnum>& OutArray)
	{
		if (JsonObject.TryGetEnumArrayField<TEnum>(FieldName, /*out*/ OutArray))
		{
			return true;
		}
		else if (JsonObject.TryGetEnumArrayField<TEnum>(DeprecatedFieldName, /*out*/ OutArray))
		{
			//@TODO: Warn about deprecated field fallback?
			return true;
		}
		else
		{
			return false;
		}
	}
}
