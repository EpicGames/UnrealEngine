// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalizationDescriptor.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"
#include "JsonUtils/JsonObjectArrayUpdater.h"
#include "JsonUtils/JsonConversion.h"
#include "RapidJsonPluginLoading.h"

#define LOCTEXT_NAMESPACE "LocalizationDescriptor"

namespace LocalizationDescriptor
{
	FString GetDescriptorKey(const FLocalizationTargetDescriptor& Descriptor)
	{
		return Descriptor.Name;
	}

	bool TryGetDescriptorJsonObjectKey(const FJsonObject& JsonObject, FString& OutKey)
	{
		return JsonObject.TryGetStringField(TEXT("Name"), OutKey);
	}

	void UpdateDescriptorJsonObject(const FLocalizationTargetDescriptor& Descriptor, FJsonObject& JsonObject)
	{
		Descriptor.UpdateJson(JsonObject);
	}
}

ELocalizationTargetDescriptorLoadingPolicy::Type ELocalizationTargetDescriptorLoadingPolicy::FromString(const TCHAR *String)
{
	ELocalizationTargetDescriptorLoadingPolicy::Type TestType = (ELocalizationTargetDescriptorLoadingPolicy::Type)0;
	for(; TestType < ELocalizationTargetDescriptorLoadingPolicy::Max; TestType = (ELocalizationTargetDescriptorLoadingPolicy::Type)(TestType + 1))
	{
		const TCHAR *TestString = ToString(TestType);
		if (FCString::Stricmp(String, TestString) == 0)
		{
			break;
		}
	}
	return TestType;
}

const TCHAR* ELocalizationTargetDescriptorLoadingPolicy::ToString(const ELocalizationTargetDescriptorLoadingPolicy::Type Value)
{
	switch (Value)
	{
	case Never:
		return TEXT("Never");

	case Always:
		return TEXT("Always");

	case Editor:
		return TEXT("Editor");

	case Game:
		return TEXT("Game");

	case PropertyNames:
		return TEXT("PropertyNames");

	case ToolTips:
		return TEXT("ToolTips");

	default:
		ensureMsgf(false, TEXT("ELocalizationTargetDescriptorLoadingPolicy::ToString - Unrecognized ELocalizationTargetDescriptorLoadingPolicy value: %i"), Value);
		return nullptr;
	}
}

ELocalizationConfigGenerationPolicy::Type ELocalizationConfigGenerationPolicy::FromString(const TCHAR* String)
{
	ELocalizationConfigGenerationPolicy ::Type TestType = (ELocalizationConfigGenerationPolicy::Type)0;
	for (; TestType < ELocalizationConfigGenerationPolicy::Max; TestType = (ELocalizationConfigGenerationPolicy::Type)(TestType + 1))
	{
		const TCHAR* TestString = ToString(TestType);
		if (FCString::Stricmp(String, TestString) == 0)
		{
			break;
		}
	}
	return TestType;
}

const TCHAR* ELocalizationConfigGenerationPolicy::ToString(const ELocalizationConfigGenerationPolicy::Type Value)
{
	switch (Value)
	{
	case Never:
		return TEXT("Never");

	case Auto:
		return TEXT("Auto");

	case User:
		return TEXT("User");

	default:
		ensureMsgf(false, TEXT("ELocalizationTargetDescriptorGenerationPolicy ::ToString - Unrecognized ELocalizationTargetDescriptorGenerationPolicy value: %i"), Value);
		return nullptr;
	}
}

FLocalizationTargetDescriptor::FLocalizationTargetDescriptor(FString InName, ELocalizationTargetDescriptorLoadingPolicy::Type InLoadingPolicy, ELocalizationConfigGenerationPolicy::Type InGenerationPolicy)
	: Name(MoveTemp(InName))
	, LoadingPolicy(InLoadingPolicy)
	, ConfigGenerationPolicy(InGenerationPolicy)
{
}

TOptional<FText> UE::Projects::Private::Read(UE::Json::FConstObject InObject, FLocalizationTargetDescriptor& Result)
{
	// Read the target name
	if (!TryGetStringField(InObject, TEXT("Name"), Result.Name))
	{
		return LOCTEXT("TargetWithoutAName", "Found a 'Localization Target' entry with a missing 'Name' field");
		}

	// Read the target loading policy
	FString LoadingPolicyValue;
	if (TryGetStringField(InObject, TEXT("LoadingPolicy"), LoadingPolicyValue))
	{
		Result.LoadingPolicy = ELocalizationTargetDescriptorLoadingPolicy::FromString(*LoadingPolicyValue);
		if (Result.LoadingPolicy == ELocalizationTargetDescriptorLoadingPolicy::Max)
		{
			return FText::Format(LOCTEXT("TargetWithInvalidLoadingPolicy", "Localization Target entry '{0}' specified an unrecognized target LoadingPolicy '{1}'"), FText::FromString(Result.Name), FText::FromString(LoadingPolicyValue));
		}
	}

		// Read the target generation policy
	FString ConfigGenerationPolicyValue;
	if (TryGetStringField(InObject, TEXT("ConfigGenerationPolicy"), ConfigGenerationPolicyValue))
	{
		Result.ConfigGenerationPolicy = ELocalizationConfigGenerationPolicy::FromString(*ConfigGenerationPolicyValue);
		if (Result.ConfigGenerationPolicy == ELocalizationConfigGenerationPolicy::Max)
		{
			return FText::Format(LOCTEXT("TargetWithInvalidGenerationPolicy", "Localization Target entry '{0}' specified an unrecognized target GenerationPolicy'{1}'"), FText::FromString(Result.Name), FText::FromString(ConfigGenerationPolicyValue));
			}
		}
	// If we can't find the value, GenerationPolicy already defaults to Never so it's ok 

	return {};
}

bool FLocalizationTargetDescriptor::Read(const FJsonObject& InObject, FText* OutFailReason /*= nullptr*/)
{
	return UE::Projects::Private::ReadFromDefaultJson(InObject, *this, OutFailReason);
}

bool FLocalizationTargetDescriptor::Read(const FJsonObject& InObject, FText& OutFailReason)
{
	return UE::Projects::Private::ReadFromDefaultJson(InObject, *this, &OutFailReason);
}

bool FLocalizationTargetDescriptor::ReadArray(const FJsonObject& InObject, const TCHAR* InName, TArray<FLocalizationTargetDescriptor>& OutTargets, FText* OutFailReason /*= nullptr*/)
{
	return UE::Projects::Private::ReadArrayFromDefaultJson(InObject, InName, OutTargets, OutFailReason);
}

bool FLocalizationTargetDescriptor::ReadArray(const FJsonObject& InObject, const TCHAR* InName, TArray<FLocalizationTargetDescriptor>& OutTargets, FText& OutFailReason)
{
	return UE::Projects::Private::ReadArrayFromDefaultJson(InObject, InName, OutTargets, &OutFailReason);
}

void FLocalizationTargetDescriptor::Write(TJsonWriter<>& Writer) const
{
	TSharedRef<FJsonObject> DescriptorJsonObject = MakeShared<FJsonObject>();
	UpdateJson(*DescriptorJsonObject);

	FJsonSerializer::Serialize(DescriptorJsonObject, Writer);
}

void FLocalizationTargetDescriptor::UpdateJson(FJsonObject& JsonObject) const
{
	JsonObject.SetStringField(TEXT("Name"), Name);
	JsonObject.SetStringField(TEXT("LoadingPolicy"), FString(ELocalizationTargetDescriptorLoadingPolicy::ToString(LoadingPolicy)));
	JsonObject.SetStringField(TEXT("ConfigGenerationPolicy"), FString(ELocalizationConfigGenerationPolicy::ToString(ConfigGenerationPolicy)));
}

void FLocalizationTargetDescriptor::WriteArray(TJsonWriter<>& Writer, const TCHAR* ArrayName, const TArray<FLocalizationTargetDescriptor>& Descriptors)
{
	if (Descriptors.Num() > 0)
	{
		Writer.WriteArrayStart(ArrayName);
		for (const FLocalizationTargetDescriptor& Descriptor : Descriptors)
		{
			Descriptor.Write(Writer);
		}
		Writer.WriteArrayEnd();
	}
}

void FLocalizationTargetDescriptor::UpdateArray(FJsonObject& JsonObject, const TCHAR* ArrayName, const TArray<FLocalizationTargetDescriptor>& Descriptors)
{
	typedef FJsonObjectArrayUpdater<FLocalizationTargetDescriptor, FString> FLocTargetDescJsonArrayUpdater;

	FLocTargetDescJsonArrayUpdater::Execute(
		JsonObject, ArrayName, Descriptors,
		FLocTargetDescJsonArrayUpdater::FGetElementKey::CreateStatic(LocalizationDescriptor::GetDescriptorKey),
		FLocTargetDescJsonArrayUpdater::FTryGetJsonObjectKey::CreateStatic(LocalizationDescriptor::TryGetDescriptorJsonObjectKey),
		FLocTargetDescJsonArrayUpdater::FUpdateJsonObject::CreateStatic(LocalizationDescriptor::UpdateDescriptorJsonObject));
}

bool FLocalizationTargetDescriptor::ShouldLoadLocalizationTarget() const
{
	switch (LoadingPolicy)
	{
	case ELocalizationTargetDescriptorLoadingPolicy::Never:
		return false;

	case ELocalizationTargetDescriptorLoadingPolicy::Always:
		return true;

	case ELocalizationTargetDescriptorLoadingPolicy::Editor:
		return WITH_EDITOR;

	case ELocalizationTargetDescriptorLoadingPolicy::Game:
		return FApp::IsGame() || FTextLocalizationManager::Get().ShouldForceLoadGameLocalization();

	case ELocalizationTargetDescriptorLoadingPolicy::PropertyNames:
#if WITH_EDITOR
		{
			bool bShouldUseLocalizedPropertyNames = false;
			if (!GConfig->GetBool(TEXT("Internationalization"), TEXT("ShouldUseLocalizedPropertyNames"), bShouldUseLocalizedPropertyNames, GEditorSettingsIni))
			{
				GConfig->GetBool(TEXT("Internationalization"), TEXT("ShouldUseLocalizedPropertyNames"), bShouldUseLocalizedPropertyNames, GEngineIni);
			}
			return bShouldUseLocalizedPropertyNames;
		}
#else	// WITH_EDITOR
		return false;
#endif	// WITH_EDITOR

	case ELocalizationTargetDescriptorLoadingPolicy::ToolTips:
		return WITH_EDITOR;

	default:
		ensureMsgf(false, TEXT("FLocalizationTargetDescriptor::ShouldLoadLocalizationTarget - Unrecognized ELocalizationTargetDescriptorLoadingPolicy value: %i"), LoadingPolicy);
		return false;
	}
}

#undef LOCTEXT_NAMESPACE
