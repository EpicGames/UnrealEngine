// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModuleDescriptor.h"
#include "Misc/App.h"
#include "Misc/ScopedSlowTask.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"
#include "JsonUtils/JsonObjectArrayUpdater.h"
#include "Modules/ModuleManager.h"
#include "RapidJsonPluginLoading.h"
#include "JsonUtils/JsonConversion.h"

#define LOCTEXT_NAMESPACE "ModuleDescriptor"

namespace ModuleDescriptor
{
	FString GetModuleKey(const FModuleDescriptor& Module)
	{
		return Module.Name.ToString();
	}

	bool TryGetModuleJsonObjectKey(const FJsonObject& JsonObject, FString& OutKey)
	{
		return JsonObject.TryGetStringField(TEXT("Name"), OutKey);
	}

	void UpdateModuleJsonObject(const FModuleDescriptor& Module, FJsonObject& JsonObject)
	{
		Module.UpdateJson(JsonObject);
	}
}

ELoadingPhase::Type ELoadingPhase::FromString( const TCHAR *String )
{
	ELoadingPhase::Type TestType = (ELoadingPhase::Type)0;
	for(; TestType < ELoadingPhase::Max; TestType = (ELoadingPhase::Type)(TestType + 1))
	{
		const TCHAR *TestString = ToString(TestType);
		if(FCString::Stricmp(String, TestString) == 0)
		{
			break;
		}
	}
	return TestType;
}

const TCHAR* ELoadingPhase::ToString( const ELoadingPhase::Type Value )
{
	switch( Value )
	{
	case Default:
		return TEXT( "Default" );

	case PostDefault:
		return TEXT( "PostDefault" );

	case PreDefault:
		return TEXT( "PreDefault" );

	case PostConfigInit:
		return TEXT( "PostConfigInit" );

	case PostSplashScreen:
		return TEXT("PostSplashScreen");

	case PreEarlyLoadingScreen:
		return TEXT("PreEarlyLoadingScreen");

	case PreLoadingScreen:
		return TEXT( "PreLoadingScreen" );

	case PostEngineInit:
		return TEXT( "PostEngineInit" );

	case EarliestPossible:
		return TEXT("EarliestPossible");

	case None:
		return TEXT( "None" );

	default:
		ensureMsgf( false, TEXT( "Unrecognized ELoadingPhase value: %i" ), Value );
		return NULL;
	}
}

EHostType::Type EHostType::FromString( const TCHAR *String )
{
	EHostType::Type TestType = (EHostType::Type)0;
	for(; TestType < EHostType::Max; TestType = (EHostType::Type)(TestType + 1))
	{
		const TCHAR *TestString = ToString(TestType);
		if(FCString::Stricmp(String, TestString) == 0)
		{
			break;
		}
	}
	return TestType;
}

const TCHAR* EHostType::ToString( const EHostType::Type Value )
{
	switch( Value )
	{
		case Runtime:
			return TEXT( "Runtime" );

		case RuntimeNoCommandlet:
			return TEXT( "RuntimeNoCommandlet" );

		case RuntimeAndProgram:
			return TEXT( "RuntimeAndProgram" );

		case CookedOnly:
			return TEXT( "CookedOnly" );

		case UncookedOnly:
			return TEXT( "UncookedOnly" );

		case Developer:
			return TEXT( "Developer" );

		case DeveloperTool:
			return TEXT( "DeveloperTool" );

		case Editor:
			return TEXT( "Editor" );

		case EditorNoCommandlet:
			return TEXT( "EditorNoCommandlet" );

		case EditorAndProgram:
			return TEXT( "EditorAndProgram" );

		case Program:
			return TEXT("Program");

		case ServerOnly:
			return TEXT("ServerOnly");

		case ClientOnly:
			return TEXT("ClientOnly");

		case ClientOnlyNoCommandlet:
			return TEXT("ClientOnlyNoCommandlet");

		default:
			ensureMsgf( false, TEXT( "Unrecognized EModuleType value: %i" ), Value );
			return NULL;
	}
}

FModuleDescriptor::FModuleDescriptor(const FName InName, EHostType::Type InType, ELoadingPhase::Type InLoadingPhase)
	: Name(InName)
	, Type(InType)
	, LoadingPhase(InLoadingPhase)
	, bHasExplicitPlatforms(false)
{
}


TOptional<FText> UE::Projects::Private::Read(UE::Json::FConstObject Object, FModuleDescriptor& Out)
{
	// Read the module name
	FString NameValue;
	if (!TryGetStringField(Object, TEXT("Name"), NameValue))
	{
		return LOCTEXT("ModuleWithoutAName", "Found a 'Module' entry with a missing 'Name' field");
	}
	Out.Name = FName(*NameValue);

	// Read the module type
	FString TypeValue;
	if (!TryGetStringField(Object, TEXT("Type"), TypeValue))
	{
		return FText::Format( LOCTEXT( "ModuleWithoutAType", "Found Module entry '{0}' with a missing 'Type' field" ), FText::FromName(Out.Name));
	}

	Out.Type = EHostType::FromString(*TypeValue);
	if (Out.Type == EHostType::Max)
	{
		return FText::Format( LOCTEXT( "ModuleWithInvalidType", "Module entry '{0}' specified an unrecognized module Type '{1}'" ), FText::FromName(Out.Name), FText::FromString(TypeValue) );
	}

	// Read the loading phase
	FString LoadingPhaseValue;
	if (TryGetStringField(Object, TEXT("LoadingPhase"), LoadingPhaseValue))
	{
		Out.LoadingPhase = ELoadingPhase::FromString(*LoadingPhaseValue);
		if(Out.LoadingPhase == ELoadingPhase::Max)
		{
			return FText::Format( LOCTEXT( "ModuleWithInvalidLoadingPhase", "Module entry '{0}' specified an unrecognized module LoadingPhase '{1}'" ), FText::FromName(Out.Name), FText::FromString(LoadingPhaseValue) );
		}
	}

	// Read the allow/deny lists for platforms
	TryGetStringArrayFieldWithDeprecatedFallback(Object, TEXT("PlatformAllowList"), TEXT("WhitelistPlatforms"), /*out*/ Out.PlatformAllowList);
	TryGetStringArrayFieldWithDeprecatedFallback(Object, TEXT("PlatformDenyList"), TEXT("BlacklistPlatforms"), /*out*/ Out.PlatformDenyList);
	TryGetBoolField(Object, TEXT("HasExplicitPlatforms"), Out.bHasExplicitPlatforms);

	// Read the allow/deny lists for targets
	TryGetEnumArrayFieldWithDeprecatedFallback(Object, TEXT("TargetAllowList"), TEXT("WhitelistTargets"), /*out*/ Out.TargetAllowList);
	TryGetEnumArrayFieldWithDeprecatedFallback(Object, TEXT("TargetDenyList"), TEXT("BlacklistTargets"), /*out*/ Out.TargetDenyList);

	// Read the allow/deny lists for target configurations
	TryGetEnumArrayFieldWithDeprecatedFallback(Object, TEXT("TargetConfigurationAllowList"), TEXT("WhitelistTargetConfigurations"), /*out*/ Out.TargetConfigurationAllowList);
	TryGetEnumArrayFieldWithDeprecatedFallback(Object, TEXT("TargetConfigurationDenyList"), TEXT("BlacklistTargetConfigurations"), /*out*/ Out.TargetConfigurationDenyList);

	// Read the allow/deny lists for programs
	TryGetStringArrayFieldWithDeprecatedFallback(Object, TEXT("ProgramAllowList"), TEXT("WhitelistPrograms"), /*out*/ Out.ProgramAllowList);
	TryGetStringArrayFieldWithDeprecatedFallback(Object, TEXT("ProgramDenyList"), TEXT("BlacklistPrograms"), /*out*/ Out.ProgramDenyList);

	// Read the additional dependencies
	TryGetStringArrayField(Object, TEXT("AdditionalDependencies"), Out.AdditionalDependencies);
	
	// Read the allow/deny list for game targets
	TryGetStringArrayField(Object, TEXT("GameTargetAllowList"), Out.GameTargetAllowList);
	TryGetStringArrayField(Object, TEXT("GameTargetDenyList"), Out.GameTargetDenyList);
	
	// Read the platform architecture allow/deny lists
	auto ReadPlatformArchitectureList = [&Out, Object]( const TCHAR* FieldName, TMap<FString,TArray<FString>>& OutPlatformArchitectureList ) -> TOptional<FText>
	{
		TArray<FString> PlatformArchitecturePairs;
		TryGetStringArrayField(Object, FieldName, PlatformArchitecturePairs);
		for (const FString& PlatformArchitecturePair : PlatformArchitecturePairs)
		{
			FString Platform, Architecture;
			if (!PlatformArchitecturePair.Split(TEXT(":"), &Platform, &Architecture))
			{
				return FText::Format( LOCTEXT( "ModuleWithInvalidPlatformArchitecture", "Module entry '{0}' specified an unrecognized Platform:Architecture pair '{1}'" ), FText::FromName(Out.Name), FText::FromString(PlatformArchitecturePair) );
			}

			OutPlatformArchitectureList.FindOrAdd(Platform).AddUnique(Architecture);
		}

		return {};
	};

	TOptional<FText> ErrorResult = ReadPlatformArchitectureList(TEXT("PlatformArchitectureAllowList"), Out.PlatformArchitectureAllowList);
	if (ErrorResult)
	{
		return ErrorResult;
	}

	ErrorResult = ReadPlatformArchitectureList(TEXT("PlatformArchitectureDenyList"), Out.PlatformArchitectureDenyList);

	if (ErrorResult)
	{
		return ErrorResult;
	}

	return {};
}
	
bool FModuleDescriptor::Read(const FJsonObject& Object, FText* OutFailReason /*= nullptr*/)
{
	return UE::Projects::Private::ReadFromDefaultJson(Object, *this, OutFailReason);
}

bool FModuleDescriptor::Read(const FJsonObject& Object, FText& OutFailReason)
{
	return UE::Projects::Private::ReadFromDefaultJson(Object, *this, &OutFailReason);
}

bool FModuleDescriptor::ReadArray(const FJsonObject& Object, const TCHAR* Name, TArray<FModuleDescriptor>& OutModules, FText* OutFailReason /*= nullptr*/)
{
	return UE::Projects::Private::ReadArrayFromDefaultJson(Object, Name, OutModules, OutFailReason);
}

bool FModuleDescriptor::ReadArray(const FJsonObject& Object, const TCHAR* Name, TArray<FModuleDescriptor>& OutModules, FText& OutFailReason)
{
	return UE::Projects::Private::ReadArrayFromDefaultJson(Object, Name, OutModules, &OutFailReason);
}

void FModuleDescriptor::Write(TJsonWriter<>& Writer) const
{
	TSharedRef<FJsonObject> ModuleJsonObject = MakeShared<FJsonObject>();
	UpdateJson(*ModuleJsonObject);

	FJsonSerializer::Serialize(ModuleJsonObject, Writer);
}

void FModuleDescriptor::UpdateJson(FJsonObject& JsonObject) const
{
	JsonObject.SetStringField(TEXT("Name"), Name.ToString());
	JsonObject.SetStringField(TEXT("Type"), FString(EHostType::ToString(Type)));
	JsonObject.SetStringField(TEXT("LoadingPhase"), FString(ELoadingPhase::ToString(LoadingPhase)));

	if (PlatformAllowList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> PlatformAllowListValues;
		for (const FString& Entry : PlatformAllowList)
		{
			PlatformAllowListValues.Add(MakeShareable(new FJsonValueString(Entry)));
		}
		JsonObject.SetArrayField(TEXT("PlatformAllowList"), PlatformAllowListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("PlatformAllowList"));
	}

	if (PlatformDenyList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> PlatformDenyListValues;
		for (const FString& Entry : PlatformDenyList)
		{
			PlatformDenyListValues.Add(MakeShareable(new FJsonValueString(Entry)));
		}
		JsonObject.SetArrayField(TEXT("PlatformDenyList"), PlatformDenyListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("PlatformDenyList"));
	}


	auto GetPlatformArchitectureListValues = []( const TMap<FString,TArray<FString>>& PlatformArchitectureList )
	{
		TArray<TSharedPtr<FJsonValue>> ListValues;
		for (const TPair<FString,TArray<FString>>& Pairs : PlatformArchitectureList)
		{
			for (const FString& Value : Pairs.Value)
			{
				FString PlatformArchitectureValue = Pairs.Key + TEXT(":") + Value;
				ListValues.Add(MakeShareable(new FJsonValueString(PlatformArchitectureValue)));
			}
		}

		return ListValues;
	};

	if (PlatformArchitectureAllowList.Num() > 0)
	{
		JsonObject.SetArrayField(TEXT("PlatformArchitectureAllowList"), GetPlatformArchitectureListValues(PlatformArchitectureAllowList));
	}
	else
	{
		JsonObject.RemoveField(TEXT("PlatformArchitectureAllowList"));
	}
		
	if (PlatformArchitectureDenyList.Num() > 0)
	{
		JsonObject.SetArrayField(TEXT("PlatformArchitectureDenyList"), GetPlatformArchitectureListValues(PlatformArchitectureDenyList));
	}
	else
	{
		JsonObject.RemoveField(TEXT("PlatformArchitectureDenyList"));
	}


	if (TargetAllowList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> TargetAllowListValues;
		for (EBuildTargetType Target : TargetAllowList)
		{
			TargetAllowListValues.Add(MakeShareable(new FJsonValueString(LexToString(Target))));
		}
		JsonObject.SetArrayField(TEXT("TargetAllowList"), TargetAllowListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("TargetAllowList"));
	}

	if (TargetDenyList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> TargetDenyListValues;
		for (EBuildTargetType Target : TargetDenyList)
		{
			TargetDenyListValues.Add(MakeShareable(new FJsonValueString(LexToString(Target))));
		}
		JsonObject.SetArrayField(TEXT("TargetDenyList"), TargetDenyListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("TargetDenyList"));
	}

	if (TargetConfigurationAllowList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> TargetConfigurationAllowListValues;
		for (EBuildConfiguration Config : TargetConfigurationAllowList)
		{
			TargetConfigurationAllowListValues.Add(MakeShareable(new FJsonValueString(LexToString(Config))));
		}
		JsonObject.SetArrayField(TEXT("TargetConfigurationAllowList"), TargetConfigurationAllowListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("TargetConfigurationAllowList"));
	}

	if (TargetConfigurationDenyList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> TargetConfigurationDenyListValues;
		for (EBuildConfiguration Config : TargetConfigurationDenyList)
		{
			TargetConfigurationDenyListValues.Add(MakeShareable(new FJsonValueString(LexToString(Config))));
		}
		JsonObject.SetArrayField(TEXT("TargetConfigurationDenyList"), TargetConfigurationDenyListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("TargetConfigurationDenyList"));
	}

	if (ProgramAllowList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> ProgramAllowListValues;
		for (const FString& Program : ProgramAllowList)
		{
			ProgramAllowListValues.Add(MakeShareable(new FJsonValueString(Program)));
		}
		JsonObject.SetArrayField(TEXT("ProgramAllowList"), ProgramAllowListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("ProgramAllowList"));
	}

	if (ProgramDenyList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> ProgramDenyListValues;
		for (const FString& Program : ProgramDenyList)
		{
			ProgramDenyListValues.Add(MakeShareable(new FJsonValueString(Program)));
		}
		JsonObject.SetArrayField(TEXT("ProgramDenyList"), ProgramDenyListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("ProgramDenyList"));
	}
	if (GameTargetAllowList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> GameTargetAllowListValues;
		for (const FString& Program : ProgramAllowList)
		{
			GameTargetAllowListValues.Add(MakeShareable(new FJsonValueString(Program)));
		}
		JsonObject.SetArrayField(TEXT("GameTargetAllowList"), GameTargetAllowListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("GameTargetAllowList"));
	}

	if (GameTargetDenyList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> GameTargetDenyListValues;
		for (const FString& Program : ProgramDenyList)
		{
			GameTargetDenyListValues.Add(MakeShareable(new FJsonValueString(Program)));
		}
		JsonObject.SetArrayField(TEXT("GameTargetDenyList"), GameTargetDenyListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("GameTargetDenyList"));
	}
	
	if (AdditionalDependencies.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> AdditionalDependencyValues;
		for (const FString& AdditionalDependency : AdditionalDependencies)
		{
			AdditionalDependencyValues.Add(MakeShareable(new FJsonValueString(AdditionalDependency)));
		}
		JsonObject.SetArrayField(TEXT("AdditionalDependencies"), AdditionalDependencyValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("AdditionalDependencies"));
	}

	if (bHasExplicitPlatforms)
	{
		JsonObject.SetBoolField(TEXT("HasExplicitPlatforms"), bHasExplicitPlatforms);
	}
	else
	{
		JsonObject.RemoveField(TEXT("HasExplicitPlatforms"));
	}

	// Clear away deprecated fields
	JsonObject.RemoveField(TEXT("WhitelistPlatforms"));
	JsonObject.RemoveField(TEXT("BlacklistPlatforms"));
	JsonObject.RemoveField(TEXT("WhitelistTargets"));
	JsonObject.RemoveField(TEXT("BlacklistTargets"));
	JsonObject.RemoveField(TEXT("WhitelistTargetConfigurations"));
	JsonObject.RemoveField(TEXT("BlacklistTargetConfigurations"));
	JsonObject.RemoveField(TEXT("WhitelistPrograms"));
	JsonObject.RemoveField(TEXT("BlacklistPrograms"));
}

void FModuleDescriptor::WriteArray(TJsonWriter<>& Writer, const TCHAR* ArrayName, const TArray<FModuleDescriptor>& Modules)
{
	if (Modules.Num() > 0)
	{
		Writer.WriteArrayStart(ArrayName);
		for(const FModuleDescriptor& Module : Modules)
		{
			Module.Write(Writer);
		}
		Writer.WriteArrayEnd();
	}
}

void FModuleDescriptor::UpdateArray(FJsonObject& JsonObject, const TCHAR* ArrayName, const TArray<FModuleDescriptor>& Modules)
{
	typedef FJsonObjectArrayUpdater<FModuleDescriptor, FString> FModuleJsonArrayUpdater;

	FModuleJsonArrayUpdater::Execute(
		JsonObject, ArrayName, Modules,
		FModuleJsonArrayUpdater::FGetElementKey::CreateStatic(ModuleDescriptor::GetModuleKey),
		FModuleJsonArrayUpdater::FTryGetJsonObjectKey::CreateStatic(ModuleDescriptor::TryGetModuleJsonObjectKey),
		FModuleJsonArrayUpdater::FUpdateJsonObject::CreateStatic(ModuleDescriptor::UpdateModuleJsonObject));
}

bool FModuleDescriptor::IsCompiledInConfiguration(const FString& Platform, EBuildConfiguration Configuration, const FString& TargetName, EBuildTargetType TargetType, bool bBuildDeveloperTools, bool bBuildRequiresCookedData, const FString& Architecture) const
{
	// Check the platform is allowed
	if ((bHasExplicitPlatforms || PlatformAllowList.Num() > 0) && !PlatformAllowList.Contains(Platform))
	{
		return false;
	}

	// Check the platform is not denied
	if (PlatformDenyList.Contains(Platform))
	{
		return false;
	}

	// Check the platform architecture is allowed
	if (!Architecture.IsEmpty() && PlatformArchitectureAllowList.Contains(Platform) && !PlatformArchitectureAllowList[Platform].Contains(Architecture))
	{
		return false;
	}

	// Check the platform architecture is not denied
	checkf(!(PlatformArchitectureDenyList.Contains(Platform) && Architecture == TEXT("MULTI")), TEXT("PlatformArchitectureDenyList does not support %s Multi-architecture builds (%s)"), *Platform, *Name.ToString());
	if (!Architecture.IsEmpty() && PlatformArchitectureDenyList.Contains(Platform) && PlatformArchitectureDenyList[Platform].Contains(Architecture))
	{
		return false;
	}

	// Check the target is allowed
	if (TargetAllowList.Num() > 0 && !TargetAllowList.Contains(TargetType))
	{
		return false;
	}

	// Check the target is not denied
	if (TargetDenyList.Contains(TargetType))
	{
		return false;
	}

	// Check the target configuration is allowed
	if (TargetConfigurationAllowList.Num() > 0 && !TargetConfigurationAllowList.Contains(Configuration))
	{
		return false;
	}

	// Check the target configuration is not denied
	if (TargetConfigurationDenyList.Contains(Configuration))
	{
		return false;
	}

	// Special checks just for programs
	if(TargetType == EBuildTargetType::Program)
	{
		// Check the program name is allowed. Note that this behavior is slightly different to other allow/deny checks; we will allow a module of any type if it's explicitly allowed for this program.
		if(ProgramAllowList.Num() > 0)
		{
			return ProgramAllowList.Contains(TargetName);
		}
				
		// Check the program name is not denied
		if(ProgramDenyList.Contains(TargetName))
		{
			return false;
		}
	}
	else
	{
		if(GameTargetAllowList.Num() > 0 && !GameTargetAllowList.Contains(TargetName))
		{
			return false;
		}
				
		if(GameTargetDenyList.Contains(TargetName))
		{
			return false;
		}
	}
	
	// Check the module is compatible with this target.
	switch (Type)
	{
	case EHostType::Runtime:
	case EHostType::RuntimeNoCommandlet:
        return TargetType != EBuildTargetType::Program;
	case EHostType::RuntimeAndProgram:
		return true;
	case EHostType::CookedOnly:
        return bBuildRequiresCookedData;
	case EHostType::UncookedOnly:
		return !bBuildRequiresCookedData;
	case EHostType::Developer:
		return TargetType == EBuildTargetType::Editor || TargetType == EBuildTargetType::Program;
	case EHostType::DeveloperTool:
		return bBuildDeveloperTools;
	case EHostType::Editor:
	case EHostType::EditorNoCommandlet:
		return TargetType == EBuildTargetType::Editor;
	case EHostType::EditorAndProgram:
		return TargetType == EBuildTargetType::Editor || TargetType == EBuildTargetType::Program;
	case EHostType::Program:
		return TargetType == EBuildTargetType::Program;
    case EHostType::ServerOnly:
        return TargetType != EBuildTargetType::Program && TargetType != EBuildTargetType::Client;
    case EHostType::ClientOnly:
	case EHostType::ClientOnlyNoCommandlet:
        return TargetType != EBuildTargetType::Program && TargetType != EBuildTargetType::Server;
    }

	return false;
}

bool FModuleDescriptor::IsCompiledInCurrentConfiguration() const
{
	return IsCompiledInConfiguration(FPlatformMisc::GetUBTPlatform(), FApp::GetBuildConfiguration(), UE_APP_NAME, FApp::GetBuildTargetType(), !!WITH_UNREAL_DEVELOPER_TOOLS, FPlatformProperties::RequiresCookedData(), FPlatformMisc::GetUBTArchitecture());
}

bool FModuleDescriptor::IsLoadedInCurrentConfiguration() const
{
	// Check that the module is built for this configuration
	if(!IsCompiledInCurrentConfiguration())
	{
		return false;
	}

	
	// Always respect the allow/deny lists for program targets
	EBuildTargetType TargetType = FApp::GetBuildTargetType();
	if(TargetType == EBuildTargetType::Program)
	{
		const FString TargetName = UE_APP_NAME;
			
		// Check the program name is allowed. Note that this behavior is slightly different to other allow/deny list checks; we will allow a module of any type if it's explicitly allowed for this program.
		if(ProgramAllowList.Num() > 0)
		{
			return ProgramAllowList.Contains(TargetName);
		}
				
		// Check the program name is not denied
		if(ProgramDenyList.Contains(TargetName))
		{
			return false;
		}
	}
	else
	{
		const FString TargetName = UE_APP_NAME;
		
		if(GameTargetAllowList.Num() > 0 && !GameTargetAllowList.Contains(TargetName))
		{
			return false;
		}
				
		if(GameTargetDenyList.Contains(TargetName))
		{
			return false;
		}
	}

	// Check that the runtime environment allows it to be loaded
	switch (Type)
	{
	case EHostType::RuntimeAndProgram:
		#if (WITH_ENGINE || WITH_PLUGIN_SUPPORT)
			return true;
		#else
			break;
		#endif

	case EHostType::Runtime:
		#if (WITH_ENGINE || WITH_PLUGIN_SUPPORT) && !IS_PROGRAM
			return true;
		#else
			break;
		#endif
	
	case EHostType::RuntimeNoCommandlet:
		#if (WITH_ENGINE || WITH_PLUGIN_SUPPORT)  && !IS_PROGRAM
			if(!IsRunningCommandlet()) return true;
		#else
			break;
		#endif

	case EHostType::CookedOnly:
		return FPlatformProperties::RequiresCookedData();

	case EHostType::UncookedOnly:
		return !FPlatformProperties::RequiresCookedData();

	case EHostType::Developer:
		#if WITH_EDITOR || IS_PROGRAM
			return true;
		#else
			return false;
		#endif

	case EHostType::DeveloperTool:
		#if WITH_UNREAL_DEVELOPER_TOOLS
			return true;
		#else
			return false;
		#endif

	case EHostType::Editor:
		#if WITH_EDITOR
			#if !UE_EDITOR
				// SetIsRunningAsCommandlet() may be set after AppInit() via "late commandlet token" path
				ensure(LoadingPhase != ELoadingPhase::PostConfigInit && LoadingPhase != ELoadingPhase::EarliestPossible);
			#endif
			if(GIsEditor) return true;
		#endif
		break;

	case EHostType::EditorNoCommandlet:
		#if WITH_EDITOR
			#if !UE_EDITOR
				// SetIsRunningAsCommandlet() may be set after AppInit() via "late commandlet token" path
				ensure(LoadingPhase != ELoadingPhase::PostConfigInit && LoadingPhase != ELoadingPhase::EarliestPossible);
			#endif
			if(GIsEditor && !IsRunningCommandlet()) return true;
		#endif
		break;

	case EHostType::EditorAndProgram:
		#if WITH_EDITOR
			#if !UE_EDITOR
				// SetIsRunningAsCommandlet() may be set after AppInit() via "late commandlet token" path
				ensure(LoadingPhase != ELoadingPhase::PostConfigInit && LoadingPhase != ELoadingPhase::EarliestPossible);
			#endif
			return GIsEditor;
		#elif IS_PROGRAM
			return true;
		#else
			return false;
		#endif

	case EHostType::Program:
		#if WITH_PLUGIN_SUPPORT && IS_PROGRAM
			return true;
		#else
			break;
		#endif

	case EHostType::ServerOnly:
		return !FPlatformProperties::IsClientOnly();

	case EHostType::ClientOnlyNoCommandlet:
#if (WITH_ENGINE || WITH_PLUGIN_SUPPORT)  && !IS_PROGRAM
		return (!IsRunningDedicatedServer()) && (!IsRunningCommandlet());
#endif
		// the fall in the case of not having defines listed above is intentional
	case EHostType::ClientOnly:
		return !IsRunningDedicatedServer();
	
	}
	return false;
}

void FModuleDescriptor::LoadModulesForPhase(ELoadingPhase::Type LoadingPhase, const TArray<FModuleDescriptor>& Modules, TMap<FName, EModuleLoadResult>& ModuleLoadErrors)
{
	FScopedSlowTask SlowTask((float)Modules.Num());
	for (int Idx = 0; Idx < Modules.Num(); Idx++)
	{
		SlowTask.EnterProgressFrame(1);
		const FModuleDescriptor& Descriptor = Modules[Idx];

		// Don't need to do anything if this module is already loaded
		if (!FModuleManager::Get().IsModuleLoaded(Descriptor.Name))
		{
			if (LoadingPhase == Descriptor.LoadingPhase && Descriptor.IsLoadedInCurrentConfiguration())
			{
				// @todo plugin: DLL search problems.  Plugins that statically depend on other modules within this plugin may not be found?  Need to test this.

				// NOTE: Loading this module may cause other modules to become loaded, both in the engine or game, or other modules 
				//       that are part of this project or plugin.  That's totally fine.
				EModuleLoadResult FailureReason;
				IModuleInterface* ModuleInterface = FModuleManager::Get().LoadModuleWithFailureReason(Descriptor.Name, FailureReason);
				if (ModuleInterface == nullptr)
				{
					// The module failed to load. Note this in the ModuleLoadErrors list.
					ModuleLoadErrors.Add(Descriptor.Name, FailureReason);
				}
			}
		}
	}
}

void FModuleDescriptor::UnloadModulesForPhase(ELoadingPhase::Type LoadingPhase, const TArray<FModuleDescriptor>& Modules, TMap<FName, EModuleUnloadResult>& OutErrors, bool bSkipUnload /*= false*/, bool bAllowUnloadCode /*= true*/)
{
	FScopedSlowTask SlowTask((float)Modules.Num());
	for (const FModuleDescriptor& Descriptor : Modules)
	{
		SlowTask.EnterProgressFrame();

		if (LoadingPhase != Descriptor.LoadingPhase)
		{
			continue;
		}

		IModuleInterface* Module = FModuleManager::Get().GetModule(Descriptor.Name);
		if (!Module)
		{
			continue;
		}

		if (!Module->SupportsDynamicReloading())
		{
			OutErrors.Add(Descriptor.Name, EModuleUnloadResult::UnloadNotSupported);
			continue;
		}

		if (bSkipUnload)
		{
			// Useful to gather errors without actually unloading
			continue;
		}

		Module->PreUnloadCallback();
		verify(FModuleManager::Get().UnloadModule(Descriptor.Name, false, bAllowUnloadCode));
	}
}

#if !IS_MONOLITHIC
bool FModuleDescriptor::CheckModuleCompatibility(const TArray<FModuleDescriptor>& Modules, TArray<FString>& OutIncompatibleFiles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FModuleDescriptor::CheckModuleCompatibility);

	FModuleManager& ModuleManager = FModuleManager::Get();

	bool bResult = true;
	for (const FModuleDescriptor& Module : Modules)
	{
		if (Module.IsCompiledInCurrentConfiguration() && !ModuleManager.IsModuleUpToDate(Module.Name))
		{
			OutIncompatibleFiles.Add(Module.Name.ToString());
			bResult = false;
		}
	}
	return bResult;
}
#endif

#undef LOCTEXT_NAMESPACE
