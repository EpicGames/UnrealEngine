// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginDescriptor.h"
#include "Misc/FileHelper.h"
#include "HAL/FileManager.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "ProjectDescriptor.h"
#include "JsonUtils/JsonConversion.h"
#include "RapidJsonPluginLoading.h"

#define LOCTEXT_NAMESPACE "PluginDescriptor"

namespace UE::PluginDescriptor::Internal
{
	bool ReadFile(const TCHAR* FileName, FString& Text, FText* OutFailReason/* = nullptr*/)
	{
		if (!FFileHelper::LoadFileToString(Text, FileName))
		{
			if (OutFailReason)
			{
				*OutFailReason = FText::Format(LOCTEXT("FailedToLoadDescriptorFile", "Failed to open descriptor file '{0}'"), FText::FromStringView(FileName));
			}
			return false;
		}
		return true;
	}

	bool WriteFile(const TCHAR* FileName, const FString& Text, FText* OutFailReason/* = nullptr*/)
	{
		if (!FFileHelper::SaveStringToFile(Text, FileName))
		{
			if (OutFailReason)
			{
				*OutFailReason = FText::Format(LOCTEXT("FailedToWriteDescriptorFile", "Failed to write plugin descriptor file '{0}'. Perhaps the file is Read-Only?"), FText::FromStringView(FileName));
			}
			return false;
		}
		return true;
	}

	TSharedPtr<FJsonObject> DeserializeJson(const FString& Text, FText* OutFailReason/* = nullptr*/)
	{
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Text);
		if (!FJsonSerializer::Deserialize(Reader, JsonObject) || !JsonObject.IsValid())
		{
			if (OutFailReason)
			{
				*OutFailReason = FText::Format(LOCTEXT("FailedToReadDescriptorFile", "Failed to read file. {0}"), FText::FromString(Reader->GetErrorMessage()));
			}
			return TSharedPtr<FJsonObject>();
		}
		return JsonObject;
	}
}

/**
 * Version numbers for plugin descriptors. These version numbers are not generally needed; serialization from JSON attempts to be tolerant of missing/added fields.
 */ 
enum class EPluginDescriptorVersion : uint8
{
	Invalid = 0,
	Initial = 1,
	NameHash = 2,
	ProjectPluginUnification = 3,
	// !!!!!!!!!! IMPORTANT: Remember to also update LatestPluginDescriptorFileVersion in Plugins.cs (and Plugin system documentation) when this changes!!!!!!!!!!!
	// -----<new versions can be added before this line>-------------------------------------------------
	// - this needs to be the last line (see note below)
	LatestPlusOne,
	Latest = LatestPlusOne - 1
};

const FString& FPluginDescriptor::GetFileExtension()
{
	static const FString Extension(".uplugin");
	return Extension;
}

FPluginDescriptor::FPluginDescriptor()
	: Version(0)
	, VerseScope(EVerseScope::PublicUser)
	, EnabledByDefault(EPluginEnabledByDefault::Unspecified)
	, bCanContainContent(false)
	, bCanContainVerse(false)
	, bIsBetaVersion(false)
	, bIsExperimentalVersion(false)
	, bInstalled(false)
	, bRequiresBuildPlatform(false)
	, bIsHidden(false)
	, bIsSealed(false)
	, bNoCode(false)
	, bExplicitlyLoaded(false)
	, bHasExplicitPlatforms(false)
	, bIsPluginExtension(false)
{
}

bool FPluginDescriptor::Load(const TCHAR* FileName, FText* OutFailReason /*= nullptr*/)
{
#if WITH_EDITOR
	CachedJson.Reset();
	AdditionalFieldsToWrite.Reset();
	AdditionalFieldsToRemove.Reset();
#endif // WITH_EDITOR

	bool GotText = false;
	if (CustomPluginDescriptorReaderDelegate.IsBound())
	{
		return ReadWithCustomPluginDescriptorReader(FileName, OutFailReason);
	}
	else
	{
		FString Text;
		if (UE::PluginDescriptor::Internal::ReadFile(FileName, Text, OutFailReason))
		{
			return Read(Text, OutFailReason);
		}
	}

	return false;
}

bool FPluginDescriptor::Load(const FString& FileName, FText* OutFailReason /*= nullptr*/)
{
	return Load(*FileName, OutFailReason);
}

bool FPluginDescriptor::Load(const FString& FileName, FText& OutFailReason)
{
	return Load(*FileName, &OutFailReason);
}

bool FPluginDescriptor::ReadWithCustomPluginDescriptorReader(const TCHAR* FileName, FText* OutFailReason /*= nullptr*/)
{
	using namespace UE::Json;

	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	bool RetryWithNative = true;
	if (CustomPluginDescriptorReaderDelegate.IsBound())
	{
		RetryWithNative = false;
		bool Success = CustomPluginDescriptorReaderDelegate.Execute(FileName, OutFailReason, JsonObject, RetryWithNative);
		if (!Success && !RetryWithNative)
		{
			return false;
		}
	}

	if (!JsonObject.IsValid() || RetryWithNative)
	{
		if (RetryWithNative)
		{
			// Reset the json container
			JsonObject = MakeShared<FJsonObject>();
			FString Text;
			if (UE::PluginDescriptor::Internal::ReadFile(FileName, Text, OutFailReason))
			{
				return Read(Text, OutFailReason);
			}
		}
		return false;
	}


#if WITH_EDITOR
	CachedJson.Reset();
	AdditionalFieldsToWrite.Reset();
	AdditionalFieldsToRemove.Reset();
#endif // WITH_EDITOR

	TOptional<FDocument> RapidJsonDocument = ConvertSharedJsonToRapidJsonDocument(*JsonObject);
	if (!RapidJsonDocument.IsSet())
	{
		if (OutFailReason != nullptr)
		{
			*OutFailReason = FText::Format(LOCTEXT("InvalidDescriptorFile", "Internal error or malformed plugin descriptor file. {0}"), FText::FromString(FileName));
		}
		return false;
	}

	TOptional<FConstObject> RootObject = GetRootObject(RapidJsonDocument.GetValue());
	if (RootObject.IsSet())
	{
		// Parse it as a plug-in descriptor
		if (TOptional<FText> ReadError = UE::Projects::Private::Read(*RootObject, *this))
		{
			if (OutFailReason)
			{
				*OutFailReason = MoveTemp(*ReadError);
			}
			return false;
		}


#if WITH_EDITOR
		CachedJson = JsonObject;
		AdditionalFieldsToWrite.Reset();
		AdditionalFieldsToRemove.Reset();
#endif // WITH_EDITOR
		return true;
	}
	return false;
}

bool FPluginDescriptor::Read(const FString& Text, FText* OutFailReason /*= nullptr*/)
{
	using namespace UE::Json;

#if WITH_EDITOR
	CachedJson.Reset();
	AdditionalFieldsToWrite.Reset();
	AdditionalFieldsToRemove.Reset();
#endif // WITH_EDITOR

	TValueOrError<FDocument, FParseError> ParseResult = Parse(Text);

	// Deserialize a JSON object from the string
	if (ParseResult.HasError())
	{
		if (OutFailReason)
		{
			*OutFailReason = FText::Format(LOCTEXT("FailedToReadDescriptorFile", "Failed to read file. {0}"), FText::FromString(ParseResult.GetError().CreateMessage(Text)));
		}
		return false;
	}

	FDocument Document = ParseResult.StealValue();

	TOptional<FConstObject> RootObject = GetRootObject(Document);
	if (RootObject.IsSet())
	{
		// Parse it as a plug-in descriptor
		if (TOptional<FText> ReadError = UE::Projects::Private::Read(*RootObject, *this))
		{
			if (OutFailReason)
		{
				*OutFailReason = MoveTemp(*ReadError);
			}			
			return false;
		}

#if WITH_EDITOR
		CachedJson = ConvertRapidJsonToSharedJsonObject(*RootObject);
			AdditionalFieldsToWrite.Reset();
			AdditionalFieldsToRemove.Reset();
#endif // WITH_EDITOR
			return true;

	}
	return false;
}

bool FPluginDescriptor::Read(const FString& Text, FText& OutFailReason)
{
	return Read(Text, &OutFailReason);
}

TOptional<FText> UE::Projects::Private::Read(UE::Json::FConstObject Object, FPluginDescriptor& Result)
{
	// Read the file version
	int32 FileVersionInt32 = (int32)EPluginDescriptorVersion::Invalid;
	if(!TryGetNumberField(Object, TEXT("FileVersion"), FileVersionInt32))
	{
		if(!TryGetNumberField(Object, TEXT("PluginFileVersion"), FileVersionInt32))
		{
			return LOCTEXT("InvalidProjectFileVersion", "File does not have a valid 'FileVersion' number.");
		}
	}

	// Check that it's within range
	EPluginDescriptorVersion PluginFileVersion = (EPluginDescriptorVersion)FileVersionInt32;
	if ((PluginFileVersion <= EPluginDescriptorVersion::Invalid) || (PluginFileVersion > EPluginDescriptorVersion::Latest))
	{
		const FText ReadVersionText = FText::FromString(FString::Printf(TEXT("%d"), (int32)PluginFileVersion));
		const FText LatestVersionText = FText::FromString(FString::Printf(TEXT("%d"), (int32)EPluginDescriptorVersion::Latest));
		return FText::Format(LOCTEXT("ProjectFileVersionTooLarge", "File appears to be in a newer version ({0}) of the file format that we can load (max version: {1})."), ReadVersionText, LatestVersionText);
	}

	// Read the other fields
	TryGetNumberField(Object, TEXT("Version"), Result.Version);
	TryGetStringField(Object, TEXT("VersionName"), Result.VersionName);
	TryGetStringField(Object, TEXT("FriendlyName"), Result.FriendlyName);
	TryGetStringField(Object, TEXT("Description"), Result.Description);

	if (!TryGetStringField(Object, TEXT("Category"), Result.Category))
	{
		// Category used to be called CategoryPath in .uplugin files
		TryGetStringField(Object, TEXT("CategoryPath"), Result.Category);
	}
        
	// Due to a difference in command line parsing between Windows and Mac, we shipped a few Mac samples containing
	// a category name with escaped quotes. Remove them here to make sure we can list them in the right category.
	if (Result.Category.Len() >= 2 && Result.Category.StartsWith(TEXT("\""), ESearchCase::CaseSensitive) && Result.Category.EndsWith(TEXT("\""), ESearchCase::CaseSensitive))
	{
		Result.Category.MidInline(1, Result.Category.Len() - 2, EAllowShrinking::No);
	}

	TryGetStringField(Object, TEXT("CreatedBy"), Result.CreatedBy);
	TryGetStringField(Object, TEXT("CreatedByURL"), Result.CreatedByURL);
	TryGetStringField(Object, TEXT("DocsURL"), Result.DocsURL);
	TryGetStringField(Object, TEXT("MarketplaceURL"), Result.MarketplaceURL);
	TryGetStringField(Object, TEXT("SupportURL"), Result.SupportURL);
	TryGetStringField(Object, TEXT("EngineVersion"), Result.EngineVersion);
	TryGetStringField(Object, TEXT("EditorCustomVirtualPath"), Result.EditorCustomVirtualPath);
	TryGetStringArrayField(Object, TEXT("SupportedTargetPlatforms"), Result.SupportedTargetPlatforms);
	TryGetStringArrayField(Object, TEXT("SupportedPrograms"), Result.SupportedPrograms);
	TryGetBoolField(Object, TEXT("bIsPluginExtension"), Result.bIsPluginExtension);

	if (TOptional<FText> ReadError = ReadArray(Object, TEXT("Modules"), Result.Modules))
	{
		return ReadError;
	}

	if (TOptional<FText> ReadError = ReadArray(Object, TEXT("LocalizationTargets"), Result.LocalizationTargets))
	{
		return ReadError;
	}

	TryGetStringField(Object, TEXT("VersePath"), Result.VersePath);
	TryGetStringField(Object, TEXT("DeprecatedEngineVersion"), Result.DeprecatedEngineVersion);

	// Read the Verse scope
	Json::FValue::ConstMemberIterator VerseScopeValue = Object.FindMember(TEXT("VerseScope"));
	if (VerseScopeValue != Object.MemberEnd() && VerseScopeValue->value.IsString())
	{
		if (TOptional<EVerseScope::Type> MaybeVerseScope = EVerseScope::FromString(VerseScopeValue->value.GetString()))
		{
			Result.VerseScope = *MaybeVerseScope;
		}
		else
		{
			return FText::Format(LOCTEXT("PluginWithInvalidVerseScope", "Plugin entry 'VerseScope' specified an unrecognized value '{0}'"), FText::FromString(VerseScopeValue->value.GetString()));
		}
	}

	// Read the Verse version.
	Json::FValue::ConstMemberIterator VerseVersionValue = Object.FindMember(TEXT("VerseVersion"));
	if (VerseVersionValue != Object.MemberEnd())
	{
		int32 PluginVerseVersion = 0;
		if (TryGetNumberField(Object, TEXT("VerseVersion"), PluginVerseVersion))
		{
			Result.VerseVersion = PluginVerseVersion;
		}
		else
		{
			return FText::Format(LOCTEXT("PluginWithInvalidVerseVersion", "Plugin entry 'VerseVersion' specified an unrecognized type '{0}'"), FText::FromString(Json::GetValueTypeName(VerseVersionValue->value)));
		}
	}

	TryGetBoolField(Object, TEXT("EnableSceneGraph"), Result.bEnableSceneGraph);

	TryGetBoolField(Object, TEXT("EnableSceneGraphAnimation"), Result.bEnableSceneGraphAnimation);
	
	TryGetBoolField(Object, TEXT("EnableVerseAssetReflection"), Result.bEnableVerseAssetReflection);

	TryGetBoolField(Object, TEXT("EnableIAD"), Result.bEnableIAD);
	TryGetBoolField(Object, TEXT("EnableIAS"), Result.bEnableIAS);

	bool bEnabledByDefault{};
	if(TryGetBoolField(Object, TEXT("EnabledByDefault"), bEnabledByDefault))
	{
		Result.EnabledByDefault = bEnabledByDefault ? EPluginEnabledByDefault::Enabled : EPluginEnabledByDefault::Disabled;
	}

	TryGetBoolField(Object, TEXT("CanContainContent"), Result.bCanContainContent);
	TryGetBoolField(Object, TEXT("CanContainVerse"), Result.bCanContainVerse);
	TryGetBoolField(Object, TEXT("NoCode"), Result.bNoCode);
	TryGetBoolField(Object, TEXT("NoGameFeatureContent"), Result.bNoGameFeatureContent);
	TryGetBoolField(Object, TEXT("IsBetaVersion"), Result.bIsBetaVersion);
	TryGetBoolField(Object, TEXT("IsExperimentalVersion"), Result.bIsExperimentalVersion);
	TryGetBoolField(Object, TEXT("Installed"), Result.bInstalled);
	TryGetBoolField(Object, TEXT("RequiresBuildPlatform"), Result.bRequiresBuildPlatform);
	TryGetBoolField(Object, TEXT("Hidden"), Result.bIsHidden);
	TryGetBoolField(Object, TEXT("Sealed"), Result.bIsSealed);
	TryGetBoolField(Object, TEXT("ExplicitlyLoaded"), Result.bExplicitlyLoaded);
	TryGetBoolField(Object, TEXT("HasExplicitPlatforms"), Result.bHasExplicitPlatforms);

	bool bCanBeUsedWithUnrealHeaderTool{};
	if(TryGetBoolField(Object, TEXT("CanBeUsedWithUnrealHeaderTool"), bCanBeUsedWithUnrealHeaderTool) && bCanBeUsedWithUnrealHeaderTool)
	{
		Result.SupportedPrograms.Add(TEXT("UnrealHeaderTool"));
	}

#if WITH_EDITOR || IS_PROGRAM
	Result.PreBuildSteps = ReadCustomBuildSteps(Object, TEXT("PreBuildSteps"));
	Result.PostBuildSteps = ReadCustomBuildSteps(Object, TEXT("PostBuildSteps"));
#endif

	if (TOptional<FText> ReadError = ReadArray(Object, TEXT("Plugins"), Result.Plugins))
	{
		return ReadError;
	}

	// Backwards compatibility support
	if (TArray<FString> DisallowedPluginNameStrings; TryGetStringArrayField(Object, TEXT("DisallowedPlugins"), DisallowedPluginNameStrings))
	{
		Result.DisallowedPlugins.Reserve(DisallowedPluginNameStrings.Num());
		for (int32 Index = 0; Index < DisallowedPluginNameStrings.Num(); ++Index)
		{
			FPluginDisallowedDescriptor& PluginDisallowedDescriptor = Result.DisallowedPlugins.AddDefaulted_GetRef();
			PluginDisallowedDescriptor.Name = DisallowedPluginNameStrings[Index];
		}
	}
	else
	{
		if (TOptional<FText> ReadError = ReadArray(Object, TEXT("DisallowedPlugins"), Result.DisallowedPlugins))
		{
			return ReadError;
		}
	}

	return {};
}

bool FPluginDescriptor::Read(const FJsonObject& Object, FText* OutFailReason /*= nullptr*/)
{
	return UE::Projects::Private::ReadFromDefaultJson(Object, *this, OutFailReason);
}

bool FPluginDescriptor::Read(const FJsonObject& Object, FText& OutFailReason)
{
	return UE::Projects::Private::ReadFromDefaultJson(Object, *this, &OutFailReason);
}

bool FPluginDescriptor::Save(const TCHAR* FileName, FText* OutFailReason /*= nullptr*/) const
{
	// Write the descriptor to text
	FString Text;
	Write(Text);

	// Save it to a file
	return UE::PluginDescriptor::Internal::WriteFile(FileName, Text, OutFailReason);
}

bool FPluginDescriptor::Save(const FString& FileName, FText* OutFailReason /*= nullptr*/) const
{
	return Save(*FileName, OutFailReason);
}

bool FPluginDescriptor::Save(const FString& FileName, FText& OutFailReason) const
{
	return Save(*FileName, &OutFailReason);
}

void FPluginDescriptor::Write(FString& Text) const
{
	// Write the contents of the descriptor to a string. Make sure the writer is destroyed so that the contents are flushed to the string.
	TSharedRef< TJsonWriter<> > WriterRef = TJsonWriterFactory<>::Create(&Text);
	TJsonWriter<>& Writer = WriterRef.Get();
	Write(Writer);
	Writer.Close();
}

void FPluginDescriptor::Write(TJsonWriter<>& Writer) const
{
	TSharedPtr<FJsonObject> PluginJsonObject = MakeShared<FJsonObject>();

#if WITH_EDITOR
	if (CachedJson.IsValid())
	{
		FJsonObject::Duplicate(/*Source=*/ CachedJson, /*Dest=*/ PluginJsonObject);
	}
#endif //if WITH_EDITOR

	UpdateJson(*PluginJsonObject);

	FJsonSerializer::Serialize(PluginJsonObject.ToSharedRef(), Writer);
}

void FPluginDescriptor::UpdateJson(FJsonObject& JsonObject) const
{
	JsonObject.SetNumberField(TEXT("FileVersion"), EProjectDescriptorVersion::Latest);
	JsonObject.SetNumberField(TEXT("Version"), Version);
	JsonObject.SetStringField(TEXT("VersionName"), VersionName);
	JsonObject.SetStringField(TEXT("FriendlyName"), FriendlyName);
	JsonObject.SetStringField(TEXT("Description"), Description);
	JsonObject.SetStringField(TEXT("Category"), Category);
	JsonObject.SetStringField(TEXT("CreatedBy"), CreatedBy);
	JsonObject.SetStringField(TEXT("CreatedByURL"), CreatedByURL);
	JsonObject.SetStringField(TEXT("DocsURL"), DocsURL);
	JsonObject.SetStringField(TEXT("MarketplaceURL"), MarketplaceURL);
	JsonObject.SetStringField(TEXT("SupportURL"), SupportURL);

	if (EngineVersion.Len() > 0)
	{
		JsonObject.SetStringField(TEXT("EngineVersion"), EngineVersion);
	}
	else
	{
		JsonObject.RemoveField(TEXT("EngineVersion"));
	}

	if (EditorCustomVirtualPath.Len() > 0)
	{
		JsonObject.SetStringField(TEXT("EditorCustomVirtualPath"), EditorCustomVirtualPath);
	}
	else
	{
		JsonObject.RemoveField(TEXT("EditorCustomVirtualPath"));
	}

	if (!VersePath.IsEmpty())
	{
		JsonObject.SetStringField(TEXT("VersePath"), VersePath);
	}
	else
	{
		JsonObject.RemoveField(TEXT("VersePath"));
	}

	if (VerseScope != EVerseScope::PublicUser)
	{
		JsonObject.SetStringField(TEXT("VerseScope"), EVerseScope::ToString(VerseScope));
	}
	else
	{
		JsonObject.RemoveField(TEXT("VerseScope"));
	}

	if (VerseVersion.IsSet())
	{
		JsonObject.SetNumberField(TEXT("VerseVersion"), VerseVersion.GetValue());
	}
	else
	{
		JsonObject.RemoveField(TEXT("VerseVersion"));
	}

	if (bEnableSceneGraph)
	{
		JsonObject.SetBoolField(TEXT("EnableSceneGraph"), bEnableSceneGraph);
	}
	else
	{
		JsonObject.RemoveField(TEXT("EnableSceneGraph"));
	}

	if (bEnableSceneGraphAnimation)
	{
		JsonObject.SetBoolField(TEXT("EnableSceneGraphAnimation"), bEnableSceneGraphAnimation);
	}
	else
	{
		JsonObject.RemoveField(TEXT("EnableSceneGraphAnimation"));
	}

	if (bEnableVerseAssetReflection)
	{
		JsonObject.SetBoolField(TEXT("EnableVerseAssetReflection"), bEnableVerseAssetReflection);
	}
	else
	{
		JsonObject.RemoveField(TEXT("EnableVerseAssetReflection"));
	}

	if (bEnableIAD)
	{
		JsonObject.SetBoolField(TEXT("EnableIAD"), bEnableIAD);
	}
	else
	{
		JsonObject.RemoveField(TEXT("EnableIAD"));
	}

	if (!bEnableIAS)
	{
		JsonObject.SetBoolField(TEXT("EnableIAS"), bEnableIAS);
	}
	else
	{
		JsonObject.RemoveField(TEXT("EnableIAS"));
	}

	if (EnabledByDefault != EPluginEnabledByDefault::Unspecified)
	{
		JsonObject.SetBoolField(TEXT("EnabledByDefault"), (EnabledByDefault == EPluginEnabledByDefault::Enabled));
	}
	else
	{
		JsonObject.RemoveField(TEXT("EnabledByDefault"));
	}

	JsonObject.SetBoolField(TEXT("CanContainContent"), bCanContainContent);
	if (bCanContainVerse)
	{
		JsonObject.SetBoolField(TEXT("CanContainVerse"), bCanContainVerse);
	}
	else
	{
		JsonObject.RemoveField(TEXT("CanContainVerse"));
	}

	if (bNoCode)
	{
		JsonObject.SetBoolField(TEXT("NoCode"), bNoCode);
	}
	if (bNoGameFeatureContent)
	{
		JsonObject.SetBoolField(TEXT("NoGameFeatureContent"), bNoGameFeatureContent);
	}
	JsonObject.SetBoolField(TEXT("IsBetaVersion"), bIsBetaVersion);
	JsonObject.SetBoolField(TEXT("IsExperimentalVersion"), bIsExperimentalVersion);
	JsonObject.SetBoolField(TEXT("Installed"), bInstalled);

	if (SupportedTargetPlatforms.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> SupportedTargetPlatformValues;
		for (const FString& SupportedTargetPlatform : SupportedTargetPlatforms)
		{
			SupportedTargetPlatformValues.Add(MakeShareable(new FJsonValueString(SupportedTargetPlatform)));
		}
		JsonObject.SetArrayField(TEXT("SupportedTargetPlatforms"), SupportedTargetPlatformValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("SupportedTargetPlatforms"));
	}

	if (SupportedPrograms.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> SupportedProgramValues;
		for (const FString& SupportedProgram : SupportedPrograms)
		{
			SupportedProgramValues.Add(MakeShareable(new FJsonValueString(SupportedProgram)));
		}
		JsonObject.SetArrayField(TEXT("SupportedPrograms"), SupportedProgramValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("SupportedPrograms"));
	}

	if (bIsPluginExtension)
	{
		JsonObject.SetBoolField(TEXT("bIsPluginExtension"), bIsPluginExtension);
	}
	else
	{
		JsonObject.RemoveField(TEXT("bIsPluginExtension"));
	}

	FModuleDescriptor::UpdateArray(JsonObject, TEXT("Modules"), Modules);

	FLocalizationTargetDescriptor::UpdateArray(JsonObject, TEXT("LocalizationTargets"), LocalizationTargets);

	if (bRequiresBuildPlatform)
	{
		JsonObject.SetBoolField(TEXT("RequiresBuildPlatform"), bRequiresBuildPlatform);
	}
	else
	{
		JsonObject.RemoveField(TEXT("RequiresBuildPlatform"));
	}

	if (bIsHidden)
	{
		JsonObject.SetBoolField(TEXT("Hidden"), bIsHidden);
	}
	else
	{
		JsonObject.RemoveField(TEXT("Hidden"));
	}

	if (bIsSealed)
	{
		JsonObject.SetBoolField(TEXT("Sealed"), bIsSealed);
	}
	else
	{
		JsonObject.RemoveField(TEXT("Sealed"));
	}

	if (bExplicitlyLoaded)
	{
		JsonObject.SetBoolField(TEXT("ExplicitlyLoaded"), bExplicitlyLoaded);
	}
	else
	{
		JsonObject.RemoveField(TEXT("ExplicitlyLoaded"));
	}

	if (bHasExplicitPlatforms)
	{
		JsonObject.SetBoolField(TEXT("HasExplicitPlatforms"), bHasExplicitPlatforms);
	}
	else
	{
		JsonObject.RemoveField(TEXT("HasExplicitPlatforms"));
	}

#if WITH_EDITOR || IS_PROGRAM
	PreBuildSteps.UpdateJson(JsonObject, TEXT("PreBuildSteps"));
	PostBuildSteps.UpdateJson(JsonObject, TEXT("PostBuildSteps"));
#endif

	FPluginReferenceDescriptor::UpdateArray(JsonObject, TEXT("Plugins"), Plugins);

	FPluginDisallowedDescriptor::UpdateArray(JsonObject, TEXT("DisallowedPlugins"), DisallowedPlugins);

#if WITH_EDITOR
	for (const auto& KVP : AdditionalFieldsToWrite)
	{
		JsonObject.SetField(KVP.Key, FJsonValue::Duplicate(KVP.Value));
	}

	for (const FString& Field : AdditionalFieldsToRemove)
	{
		JsonObject.RemoveField(Field);
	}
#endif //if WITH_EDITOR
}

bool FPluginDescriptor::UpdatePluginFile(const FString& FileName, FText* OutFailReason /*= nullptr*/) const
{
	if (IFileManager::Get().FileExists(*FileName))
	{
		// Plugin file exists so we need to read it and update it.

		FString JsonText;
		if (!UE::PluginDescriptor::Internal::ReadFile(*FileName, JsonText, OutFailReason))
		{
			return false;
		}

		TSharedPtr<FJsonObject> JsonObject = UE::PluginDescriptor::Internal::DeserializeJson(JsonText, OutFailReason);
		if (!JsonObject.IsValid())
		{
			return false;
		}

		UpdateJson(*JsonObject.Get());

		{
			TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&JsonText);
			if (!ensure(FJsonSerializer::Serialize(JsonObject.ToSharedRef(), JsonWriter)))
			{
				if (OutFailReason)
				{
					*OutFailReason = LOCTEXT("FailedToWriteDescriptor", "Failed to write plugin descriptor content");
				}
				return false;
			}
		}

#if WITH_EDITOR
		CachedJson = JsonObject;
#endif
		return UE::PluginDescriptor::Internal::WriteFile(*FileName, JsonText, OutFailReason);
	}
	else
	{
		// Plugin file doesn't exist so just write it.
		return Save(FileName, OutFailReason);
	}
}

bool FPluginDescriptor::UpdatePluginFile(const FString& FileName, FText& OutFailReason) const
{
	return UpdatePluginFile(FileName, &OutFailReason);
}

bool FPluginDescriptor::SupportsTargetPlatform(const FString& Platform) const
{
	if (bHasExplicitPlatforms)
	{
		return SupportedTargetPlatforms.Contains(Platform);
	}
	else
	{
		return SupportedTargetPlatforms.Num() == 0 || SupportedTargetPlatforms.Contains(Platform);
	}
}


FPluginDescriptor::FPluginDescriptorReaderDelegate FPluginDescriptor::CustomPluginDescriptorReaderDelegate;
#undef LOCTEXT_NAMESPACE
