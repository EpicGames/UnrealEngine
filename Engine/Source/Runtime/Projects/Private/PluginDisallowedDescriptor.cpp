// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginDisallowedDescriptor.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "JsonUtils/JsonObjectArrayUpdater.h"
#include "ProjectDescriptor.h"
#include "JsonExtensions.h"
#include "JsonUtils/JsonConversion.h"
#include "RapidJsonPluginLoading.h"

#define LOCTEXT_NAMESPACE "PluginDisallowedDescriptor"

namespace PluginDisallowedDescriptor
{
	FString GetPluginRefKey(const FPluginDisallowedDescriptor& PluginRef)
	{
		return PluginRef.Name;
	}

	bool TryGetPluginRefJsonObjectKey(const FJsonObject& JsonObject, FString& OutKey)
	{
		return JsonObject.TryGetStringField(TEXT("Name"), OutKey);
	}

	void UpdatePluginRefJsonObject(const FPluginDisallowedDescriptor& PluginRef, FJsonObject& JsonObject)
	{
		PluginRef.UpdateJson(JsonObject);
	}
}

TOptional<FText> UE::Projects::Private::Read(UE::Json::FConstObject Object, FPluginDisallowedDescriptor& Result)
{
	// Get the name
	if (!TryGetStringField(Object, TEXT("Name"), Result.Name))
	{
		return LOCTEXT("FPluginDisallowedWithoutName", "Disallowed plugin must have a 'Name' field");
		}

#if WITH_EDITOR
	TryGetStringField(Object, TEXT("Comment"), Result.Comment);
#endif //if WITH_EDITOR

	return {};
}

bool FPluginDisallowedDescriptor::Read(const FJsonObject& Object, FText* OutFailReason /*= nullptr*/)
{
	return UE::Projects::Private::ReadFromDefaultJson(Object, *this, OutFailReason);
}

void FPluginDisallowedDescriptor::Write(TJsonWriter<>& Writer) const
{
	TSharedPtr<FJsonObject> PluginRefJsonObject = MakeShared<FJsonObject>();

	UpdateJson(*PluginRefJsonObject);

	FJsonSerializer::Serialize(PluginRefJsonObject.ToSharedRef(), Writer);
}

void FPluginDisallowedDescriptor::UpdateJson(FJsonObject& JsonObject) const
{
	JsonObject.SetStringField(TEXT("Name"), Name);

#if WITH_EDITOR
	JsonObject.SetStringField(TEXT("Comment"), Comment);
#endif //if WITH_EDITOR
}

bool FPluginDisallowedDescriptor::ReadArray(const FJsonObject& Object, const TCHAR* Name, TArray<FPluginDisallowedDescriptor>& OutPlugins, FText* OutFailReason /*= nullptr*/)
{
	return UE::Projects::Private::ReadArrayFromDefaultJson(Object, Name, OutPlugins, OutFailReason);
}

void FPluginDisallowedDescriptor::WriteArray(TJsonWriter<>& Writer, const TCHAR* ArrayName, const TArray<FPluginDisallowedDescriptor>& Plugins)
{
	if (Plugins.Num() > 0)
	{
		Writer.WriteArrayStart(ArrayName);

		for (const FPluginDisallowedDescriptor& PluginRef : Plugins)
		{
			PluginRef.Write(Writer);
		}

		Writer.WriteArrayEnd();
	}
}

void FPluginDisallowedDescriptor::UpdateArray(FJsonObject& JsonObject, const TCHAR* ArrayName, const TArray<FPluginDisallowedDescriptor>& Plugins)
{
	typedef FJsonObjectArrayUpdater<FPluginDisallowedDescriptor, FString> FPluginRefJsonArrayUpdater;

	FPluginRefJsonArrayUpdater::Execute(
		JsonObject, ArrayName, Plugins,
		FPluginRefJsonArrayUpdater::FGetElementKey::CreateStatic(PluginDisallowedDescriptor::GetPluginRefKey),
		FPluginRefJsonArrayUpdater::FTryGetJsonObjectKey::CreateStatic(PluginDisallowedDescriptor::TryGetPluginRefJsonObjectKey),
		FPluginRefJsonArrayUpdater::FUpdateJsonObject::CreateStatic(PluginDisallowedDescriptor::UpdatePluginRefJsonObject));
}

#undef LOCTEXT_NAMESPACE