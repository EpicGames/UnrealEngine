// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginManager.h"
#include "ICoreUObjectPluginManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/AsciiSet.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/MessageDialog.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/ConfigUtilities.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "Misc/FileHelper.h"
#include "Misc/FeedbackContext.h"
#include "ProjectDescriptor.h"
#include "Interfaces/IProjectManager.h"
#include "Modules/ModuleManager.h"
#include "ProjectManager.h"
#include "PluginManifest.h"
#include "HAL/PlatformTime.h"
#include "Async/ParallelFor.h"
#include "Misc/ScopeRWLock.h"
#include "Algo/Accumulate.h"
#include "Algo/Find.h"
#include "Algo/Reverse.h"
#include "Containers/VersePath.h"
#include "Internationalization/TextLocalizationManager.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#if READ_TARGET_ENABLED_PLUGINS_FROM_RECEIPT
#include "TargetReceipt.h"
#endif

DEFINE_LOG_CATEGORY_STATIC( LogPluginManager, Log, All );

#define LOCTEXT_NAMESPACE "PluginManager"

#ifndef UE_DISABLE_PLUGIN_DISCOVERY
	#define UE_DISABLE_PLUGIN_DISCOVERY 0
#endif

namespace UE::PluginManager::Private
{

ICoreUObjectPluginManager* CoreUObjectPluginHandler = nullptr;

void SetCoreUObjectPluginManager(ICoreUObjectPluginManager& Handler)
{
	CoreUObjectPluginHandler = &Handler;
}

void CheckForDeprecatedPlugin(FPlugin& Plugin)
{
	if (const FString& DeprecatedEngineVersion = Plugin.GetDeprecatedEngineVersion(); !DeprecatedEngineVersion.IsEmpty())
	{
		UE_LOGF(LogPluginManager, Display, "The Plugin %ls has been marked deprecated for %ls and will be removed soon. Please update your dependencies.", *Plugin.GetName(), *DeprecatedEngineVersion);
	}
}

TArray<FString> GetPluginPathsByEnv( const TCHAR* EnvVariable )
{
	TArray<FString> AdditionalPaths;
	FString PathStr = FPlatformMisc::GetEnvironmentVariable( EnvVariable );
	if (!PathStr.IsEmpty())
	{
#if PLATFORM_WINDOWS
		const TCHAR* DirectorySeparator = TEXT(";");
#else
		const TCHAR* DirectorySeparator = TEXT(":");
#endif
		PathStr.ParseIntoArray(AdditionalPaths, DirectorySeparator, true);
	}
	return AdditionalPaths;
}

/**
 * Allow users to define additional locations to find plugins. This is to
 * support traditional DCC film pipelines where plugins can be staged
 * depending on the context. This is for only looking up compiled plugins
 * like marketplace plugins and does not imply build support which happens
 * externally.
 */
TArray<FString> GetAdditionalExternalPluginsByEnvVar()
{
#if WITH_EDITOR
	return GetPluginPathsByEnv(TEXT("UE_ADDITIONAL_PLUGIN_PATHS"));
#else
	return {};
#endif
}

}


namespace PluginSystemDefs
{
	/** File extension of plugin descriptor files.
	    NOTE: This constant exists in UnrealBuildTool code as well. */
	static const TCHAR* PluginDescriptorFileExtension = TEXT( ".uplugin" );

	/**
	 * Append all paths specified using the -PLUGIN= switch.
	 *
	 * @param  PluginPathsOut Has a value added for each -PLUGIN=value pair.
	 */
	static void GetCmdLinePluginPaths(TSet<FExternalPluginPath>& PluginPathsOut)
	{
		const TCHAR* SwitchStr = TEXT("PLUGIN=");
		const int32  SwitchLen = FCString::Strlen(SwitchStr);

		const TCHAR* SearchStr = FCommandLine::Get();
		do
		{
			FString PluginPath;

			SearchStr = FCString::Strifind(SearchStr, SwitchStr);
			if (FParse::Value(SearchStr, SwitchStr, PluginPath))
			{
				FString PluginDir = FPaths::GetPath(PluginPath);
				PluginPathsOut.Add(FExternalPluginPath{ PluginDir, EPluginExternalSource::CommandLine });
				SearchStr += SwitchLen + PluginPath.Len();
			}
			else
			{
				break;
			}
		} while (SearchStr != nullptr);
	}

	/**
	 * Retrieve a list of all paths specified via the UE_ADDITIONAL_PLUGIN_PATHS environment variable.
	 *
	 * @param  PluginPathsOut A list of values split from the environment variable.
	 */
	static void GetEnvPluginPaths(TSet<FExternalPluginPath>& PluginPathsOut)
	{
		TArray<FString> AdditionalEnvPaths = UE::PluginManager::Private::GetAdditionalExternalPluginsByEnvVar();
		for (const FString& Path : AdditionalEnvPaths)
		{
			PluginPathsOut.Add(FExternalPluginPath{ Path, EPluginExternalSource::Environment });
		}
	}
}

/**
 * Set of simple (ideally inlinable) helper methods intended to obscure how
 * `FDiscoveredPluginMap` is implemented (so that we can more easily change out it's type as needed).
 *
 * Also dictates how we separate the one "offered"  plugin out from other versions of the same plugin
 * (see DiscoveredPluginMapUtils::EInsertionType).
 */
namespace DiscoveredPluginMapUtils
{
	using FDiscoveredPluginMap = TMap<FString, TArray<TSharedRef<FPlugin>>>; // intended to match FPluginManager::FDiscoveredPluginMap

	/**
	 * Given how UE is currently structured, we can only load/enable/display one plugin for any given name.
	 * This enum is used to distinctualize how a plugin is handled -- should it be promoted as the one "offered" plugin, or suppressed?
	 */
	enum class EInsertionType
	{
		AsOfferedPlugin,   // Denotes the one plugin you want visible to the rest of the engine.
		AsSuppressedPlugin // Denotes discovered plugins that aren't reported to the rest of the engine (they're just for internal tracking and consideration).
	};

	/** Returns the one "offered" plugin from the specified entry. */
	static const TSharedRef<FPlugin>& ResolvePluginFromMapVal(const FDiscoveredPluginMap::ValueType& PluginMapValue)
	{
		// The "top" plugin in the list is the one we choose to present to the rest of UE
		return PluginMapValue.Top();
	}
	static TSharedRef<FPlugin>& ResolvePluginFromMapVal(FDiscoveredPluginMap::ValueType& PluginMapValue)
	{
		// The "top" plugin in the list is the one we choose to present to the rest of UE
		return PluginMapValue.Top();
	}

	/** Removes/Forgets all other known versions of the specified plugin entry. Retains the one "offered" version (which was prioritized and positioned for the rest of UE). */
	static void DiscardAllSuppressedVersions(FDiscoveredPluginMap::ValueType& PluginMapValue)
	{
		// The "top" (last) plugin in the list is the one we "offer". Keep it, discard the rest.
		PluginMapValue.RemoveAtSwap(0, PluginMapValue.Num()-1);
	}

	/** Returns an array of all the discovered plugins with a matching name (null if none were discovered). */
	static TArray<TSharedRef<FPlugin>>* FindAllPluginVersionsWithName(FDiscoveredPluginMap& InMap, const FDiscoveredPluginMap::KeyType& Name)
	{
		return InMap.Find(Name);
	}

	/** Returns the one "offered" plugin for the specified plugin name (if any exists). */
	static TSharedRef<FPlugin>* FindPluginInMap(FDiscoveredPluginMap& InMap, const FDiscoveredPluginMap::KeyType& Name)
	{
		FDiscoveredPluginMap::ValueType* PluginList = InMap.Find(Name);
		if (PluginList && !PluginList->IsEmpty())
		{
			// The "top" plugin in the list is the one we choose to present to the rest of UE
			return &PluginList->Top();
		}
		return nullptr;
	}

	template<typename KeyComparable>
	static TSharedRef<FPlugin>* FindPluginInMap_ByHash(FDiscoveredPluginMap& InMap, const uint32 KeyHash, const KeyComparable& Key)
	{
		FDiscoveredPluginMap::ValueType* PluginList = InMap.FindByHash(KeyHash, Key);
		if (PluginList && !PluginList->IsEmpty())
		{
			// The "top" plugin in the list is the one we choose to present to the rest of UE
			return &PluginList->Top();
		}
		return nullptr;
	}

	static TSharedRef<FPlugin>& FindPluginInMap_Checked(FDiscoveredPluginMap& InMap, const FDiscoveredPluginMap::KeyType& Name)
	{
		TSharedRef<FPlugin>* PluginPtr = FindPluginInMap(InMap, Name);
		checkf(PluginPtr, TEXT("Cannot find plugin %s"), *Name);
		return *PluginPtr;
	}

	static TSharedRef<FPlugin>* FindPluginInMap_FromFileName(FDiscoveredPluginMap& InMap, const FString& FileName)
	{
		const FString PluginName = FPaths::GetBaseFilename(FileName);
		TArray<TSharedRef<FPlugin>>* KnownVersions = DiscoveredPluginMapUtils::FindAllPluginVersionsWithName(InMap, PluginName);
		if (KnownVersions)
		{
			FString NormalizedFilenameToFind = FileName;
			FPaths::NormalizeFilename(NormalizedFilenameToFind);

			for (TSharedRef<FPlugin>& Plugin : *KnownVersions)
			{
				FString NormalizedPluginPath = Plugin->FileName;
				FPaths::NormalizeFilename(NormalizedPluginPath);
				if (NormalizedPluginPath.Equals(NormalizedFilenameToFind))
				{
					return &Plugin;
				}
			}
		}
		return nullptr;
	}

	static TSharedRef<FPlugin>* FindPluginInMap_FromDescriptor(FDiscoveredPluginMap& InMap, const FPluginReferenceDescriptor& PluginDesc)
	{
		TArray<TSharedRef<FPlugin>>* KnownVersions = DiscoveredPluginMapUtils::FindAllPluginVersionsWithName(InMap, PluginDesc.Name);
		if (KnownVersions && KnownVersions->Num() > 0)
		{
			if (PluginDesc.RequestedVersion.IsSet())
			{
				for (TSharedRef<FPlugin>& Plugin : *KnownVersions)
				{
					if (Plugin->GetDescriptor().Version == PluginDesc.RequestedVersion)
					{
						return &Plugin;
					}
				}
			}
			else
			{
				return &KnownVersions->Top();
			}
		}
		return nullptr;
	}

	static bool IsOfferedPlugin(const FDiscoveredPluginMap& InMap, const TSharedRef<FPlugin>& Plugin)
	{
		const FDiscoveredPluginMap::ValueType* PluginList = InMap.Find(Plugin->Name);
		if (PluginList && !PluginList->IsEmpty())
		{
			// The "top" plugin in the list is the one we choose to present to the rest of UE
			return Plugin == PluginList->Top();
		}
		return false;
	}

	/** Internal insersion methods used for controlling the one "offered" plugin we share with the rest of the engine. */
	static void InsertPlugin_AsOffered(FDiscoveredPluginMap::ValueType& MapEntry, const TSharedRef<FPlugin>& NewPlugin)
	{
		MapEntry.Push(NewPlugin);
	}
	static void InsertPlugin_AsSuppressed(FDiscoveredPluginMap::ValueType& MapEntry, const TSharedRef<FPlugin>& NewPlugin)
	{
		MapEntry.Insert(NewPlugin, FMath::Max(MapEntry.Num() - 1, 0));
	}

	/*
	 * Adds the specified plugin to the map, blindly assuming that the specified plugin is not a duplicate already in the map.
	 * NOTE: When specifying `EInsertionType::AsOfferedPlugin`, you're overridding the previous "offered" plugin of this name.
	 */
	static void InsertPluginIntoMap(FDiscoveredPluginMap& Map, const FDiscoveredPluginMap::KeyType& Key, const TSharedRef<FPlugin>& NewPlugin, const EInsertionType InsertionType = EInsertionType::AsOfferedPlugin)
	{
		FDiscoveredPluginMap::ValueType& PluginList = Map.FindOrAdd(Key);
		switch (InsertionType)
		{
		case EInsertionType::AsOfferedPlugin:
			InsertPlugin_AsOffered(PluginList, NewPlugin);
			break;

		case EInsertionType::AsSuppressedPlugin:
			InsertPlugin_AsSuppressed(PluginList, NewPlugin);
			break;
		}
	}

	static void RemovePluginFromMap(FDiscoveredPluginMap& InMap, const TSharedRef<FPlugin>& Plugin)
	{
		const uint32 PluginNameHash = GetTypeHash(Plugin->Name);
		FDiscoveredPluginMap::ValueType* PluginList = InMap.FindByHash(PluginNameHash, Plugin->Name);
		if (PluginList)
		{
			PluginList->Remove(Plugin);
			if (PluginList->IsEmpty())
			{
				InMap.RemoveByHash(PluginNameHash, Plugin->Name);
			}
		}
	}

	/**
	 * Given how UE is currently structured, we can only load/enable/display one plugin for a given name -- this is the "offered" plugin.
	 * This method let's us swap out the current "offered" plugin with the one we specify.
	 *
	 * NOTE: This method assumes that the specified plugin pointer is already in the map under the specified name.
	 * NOTE: After this method is ran the supplied TSharedRef<> is subject to change (use the returned value instead).
	 *
	 * @return Null if the promotion failed, otherwise returns a new TSharedRef<> referencing the plugin that `PluginPtr` was initially.
	 */
	static TSharedRef<FPlugin>* PromotePluginToOfferedVersion(FDiscoveredPluginMap& Map, TSharedRef<FPlugin>* PluginPtr)
	{
		const FString& Name = (*PluginPtr)->GetName();
		FDiscoveredPluginMap::ValueType* PluginList = Map.Find(Name);

		// Ensure the specified plugin is in the array
		if (ensureAlwaysMsgf(PluginList && PluginList->GetData() <= PluginPtr && PluginPtr < PluginList->GetData() + PluginList->Num(),
			TEXT("Specified plugin is not registered under the name '%s'. Failed to promote it."), *Name))
		{
			TSharedRef<FPlugin>* TopPluginPtr = &PluginList->Top();
			if (TopPluginPtr != PluginPtr)
			{
				// Swap pointers in the array (promoting the specified plugin to the "top").
				TSharedRef<FPlugin> OldTopPluginRef = *TopPluginPtr;
				*TopPluginPtr = *PluginPtr;
				*PluginPtr = OldTopPluginRef;
			}

			return TopPluginPtr;
		}
		return nullptr;
	}
}

namespace PluginLocalizationUtils
{
	void GetLocalizationPathsForPlugin(const IPlugin& Plugin, TArray<FString>& OutLocResPaths)
	{
		const FString PluginLocDir = Plugin.GetContentDir() / TEXT("Localization");
		for (const FLocalizationTargetDescriptor& LocTargetDesc : Plugin.GetDescriptor().LocalizationTargets)
		{
			if (LocTargetDesc.ShouldLoadLocalizationTarget())
			{
				OutLocResPaths.Add(PluginLocDir / LocTargetDesc.Name);
			}
		}
	}
}

FPlugin::FPlugin(const FString& InFileName, const FPluginDescriptor& InDescriptor, EPluginType InType)
	: Name(FPaths::GetBaseFilename(InFileName))
	, FileName(InFileName)
	, Descriptor(InDescriptor)
	, Type(InType)
	, bEnabled(false)
	, bIsMounted(false)
{

}

FPlugin::~FPlugin()
{
}

FString FPlugin::GetBaseDir() const
{
	return FPaths::GetPath(FileName);
}

TArray<FString> FPlugin::GetExtensionBaseDirs() const
{
	TArray<FString> OutDirs;
	OutDirs.Reserve(PluginExtensionFileNameList.Num());
	for (const FString& ExtensionFileName : PluginExtensionFileNameList)
	{
		OutDirs.Emplace(FPaths::GetPath(ExtensionFileName));
	}
	return OutDirs;
}

FString FPlugin::GetContentDir() const
{
	return FPaths::GetPath(FileName) / TEXT("Content");
}

FString FPlugin::GetMountedAssetPath() const
{
	FString Path;
	Path.Reserve(Name.Len() + 2);
	Path.AppendChar(TEXT('/'));
	Path.Append(Name);
	Path.AppendChar(TEXT('/'));
	return Path;
}

bool FPlugin::IsEnabledByDefault(const bool bAllowEnginePluginsEnabledByDefault) const
{
	if (Descriptor.EnabledByDefault == EPluginEnabledByDefault::Enabled)
	{
		return (GetLoadedFrom() == EPluginLoadedFrom::Project ? true : bAllowEnginePluginsEnabledByDefault);
	}
	else if (Descriptor.EnabledByDefault == EPluginEnabledByDefault::Disabled)
	{
		return false;
	}
	else
	{
		return GetLoadedFrom() == EPluginLoadedFrom::Project;
	}
}

EPluginLoadedFrom FPlugin::GetLoadedFrom() const
{
	if(Type == EPluginType::Engine || Type == EPluginType::Enterprise)
	{
		return EPluginLoadedFrom::Engine;
	}
	else
	{
		return EPluginLoadedFrom::Project;
	}
}

const FPluginDescriptor& FPlugin::GetDescriptor() const
{
	return Descriptor;
}

bool FPlugin::UpdateDescriptor(const FPluginDescriptor& NewDescriptor, FText& OutFailReason)
{
	if(!NewDescriptor.UpdatePluginFile(FileName, OutFailReason))
	{
		return false;
	}

	Descriptor = NewDescriptor;

	IPluginManager& PluginManager = IPluginManager::Get();
	if (PluginManager.OnPluginEdited().IsBound())
	{
		PluginManager.OnPluginEdited().Broadcast(*this);
	}

	return true;
}

#if WITH_EDITOR
const TSharedPtr<FJsonObject>& FPlugin::GetDescriptorJson()
{
	return Descriptor.CachedJson;
}
#endif // WITH_EDITOR

const FString& EnumToString(const EPluginType& InPluginType)
{
	// Const enum strings, special case no error.
	static const FString Engine("Engine");
	static const FString Enterprise("Enterprise");
	static const FString Project("Project");
	static const FString External("External");
	static const FString Mod("Mod");
	static const FString ErrorString("Unknown plugin type, has EPluginType been changed?");

	switch (InPluginType)
	{
	case EPluginType::Engine: return Engine;
	case EPluginType::Enterprise: return Enterprise;
	case EPluginType::Project: return Project;
	case EPluginType::External: return External;
	case EPluginType::Mod: return Mod;
	default:
		check(!"Unknown plugin type, has EPluginType been changed?");
		return ErrorString;
	}
}

namespace
{
	enum class EConfigurePluginResultCode
	{
		PluginMissing,
		PluginSealed,
		DisallowedDependency,

		Unset,
	};
}

struct FPluginManager::FConfigurePluginResultInfo
{
	EConfigurePluginResultCode Code;
	const FPluginReferenceDescriptor* PluginReference;
	TArray<const FPluginReferenceDescriptor*> ReferenceChain;

	FConfigurePluginResultInfo()
		: Code(EConfigurePluginResultCode::Unset)
		, PluginReference(nullptr)
	{
	}

	FConfigurePluginResultInfo(EConfigurePluginResultCode InCode, const FPluginReferenceDescriptor* InPluginReference)
		: Code(InCode)
		, PluginReference(InPluginReference)
	{
	}
};

FPluginManager::FPluginManager()
{
	SCOPED_BOOT_TIMING("DiscoverAllPlugins");
	DiscoverAllPlugins();

	// Register the callback that allows the text localization manager to load data for plugins
	FCoreDelegates::GatherAdditionalLocResPathsCallback.AddRaw(this, &FPluginManager::GetLocalizationPathsForEnabledPlugins);
}

FPluginManager::~FPluginManager()
{
	// NOTE: All plugins and modules should be cleaned up or abandoned by this point

	// @todo plugin: Really, we should "reboot" module manager's unloading code so that it remembers at which startup phase
	//  modules were loaded in, so that we can shut groups of modules down (in reverse-load order) at the various counterpart
	//  shutdown phases.  This will fix issues where modules that are loaded after game modules are shutdown AFTER many engine
	//  systems are already killed (like GConfig.)  Currently the only workaround is to listen to global exit events, or to
	//  explicitly unload your module somewhere.  We should be able to handle most cases automatically though!

	FCoreDelegates::GatherAdditionalLocResPathsCallback.RemoveAll(this);
}

void FPluginManager::RefreshPluginsList()
{
	// Clear out the plugins map (keep only what is enabled)
	for (FDiscoveredPluginMap::TIterator Iter(AllPlugins); Iter; ++Iter)
	{
		const TSharedRef<FPlugin>& Plugin = DiscoveredPluginMapUtils::ResolvePluginFromMapVal(Iter.Value());

		if (!Plugin->bEnabled || (Plugin->GetDescriptor().bExplicitlyLoaded && !Plugin->bIsMounted))
		{
			Iter.RemoveCurrent();
		}
		else
		{
			// Forget all the other discovered versions (which we assume aren't enabled/mounted)
			DiscoveredPluginMapUtils::DiscardAllSuppressedVersions(Iter.Value());
		}
	}

	// With the map only containing what was already enabled, fill in the rest
	// of the map (duplicate plugins are skipped using a check to the plugin's `FileName` -- see `CreatePluginObject()`)
	ReadAllPlugins(AllPlugins, PluginDiscoveryPaths);

#if WITH_EDITOR
	ModuleNameToPluginMap.Reset();
#endif //if WITH_EDITOR

	for (const FDiscoveredPluginMap::ElementType& PluginPair : AllPlugins)
	{
		const TSharedRef<FPlugin>& Plugin = DiscoveredPluginMapUtils::ResolvePluginFromMapVal(PluginPair.Value);
		if (!Plugin->bEnabled) // if it's not enabled, it's a new plugin
		{
			const uint32 PluginNameHash = GetTypeHash(Plugin->GetName());
			PluginsToConfigure.AddByHash(PluginNameHash, Plugin->GetName());
		}

#if WITH_EDITOR
		AddToModuleNameToPluginMap(Plugin);
#endif //if WITH_EDITOR
	}
}

bool VerifySinglePluginForAddOrRemove(const FPluginDescriptor& Descriptor, FText& OutFailReason)
{
	if (!ensureMsgf(!Descriptor.bIsPluginExtension, TEXT("VerifySinglePluginForAddOrRemove does not allow platform extensions. Plugin %s"), *Descriptor.FriendlyName))
	{
		OutFailReason = FText::Format(LOCTEXT("PluginIsExtension", "Plugin '{0}' is a platform extension"), FText::FromString(Descriptor.FriendlyName));
		return false;
	}

	return true;
}

bool FPluginManager::AddToPluginsList(const FString& PluginFilename, FText* OutFailReason /*= nullptr*/)
{
#if (WITH_ENGINE && !IS_PROGRAM) || WITH_PLUGIN_SUPPORT
	// No need to re-add if it already exists
	FString PluginName = FPaths::GetBaseFilename(PluginFilename);
	const TArray<TSharedRef<FPlugin>>* KnownVersions = DiscoveredPluginMapUtils::FindAllPluginVersionsWithName(AllPlugins, PluginName);
	if (KnownVersions)
	{
		FString NormalizedFilename = PluginFilename;
		FPaths::NormalizeFilename(NormalizedFilename);
		for (const TSharedRef<FPlugin>& Plugin : *KnownVersions)
		{
			FString NormalizedPluginPath = Plugin->FileName;
			FPaths::NormalizeFilename(NormalizedPluginPath);
			if (NormalizedPluginPath.Equals(NormalizedFilename))
			{
				return true;
			}
		}
	}

	// Read the plugin and load it
	FPluginDescriptor Descriptor;
	FText FailReason;
	bool bSuccess = Descriptor.Load(PluginFilename, FailReason);
	if (bSuccess)
	{
		bSuccess = VerifySinglePluginForAddOrRemove(Descriptor, FailReason);
	}

	if(bSuccess)
	{
		// Determine the plugin type
		EPluginType PluginType = EPluginType::External;
		if (PluginFilename.StartsWith(FPaths::EngineDir()))
		{
			PluginType = EPluginType::Engine;
		}
		else if (PluginFilename.StartsWith(FPaths::EnterpriseDir()))
		{
			PluginType = EPluginType::Enterprise;
		}
		else if (PluginFilename.StartsWith(FPaths::ProjectModsDir()))
		{
			PluginType = EPluginType::Mod;
		}
		else if (PluginFilename.StartsWith(FPaths::GetPath(FPaths::GetProjectFilePath())))
		{
			PluginType = EPluginType::Project;
		}

		// Create the plugin
		FDiscoveredPluginMap NewPlugins;
		TArray<TSharedRef<FPlugin>> ChildPlugins;
		CreatePluginObject(PluginFilename, Descriptor, PluginType, NewPlugins, ChildPlugins);
		ensureMsgf(ChildPlugins.Num() == 0, TEXT("AddToPluginsList does not allow plugins with bIsPluginExtension set to true. Plugin: %s"), *PluginFilename);
		ensure(NewPlugins.Num() == 1);

		// Add the loaded plugin
		TSharedRef<FPlugin>* NewPlugin = DiscoveredPluginMapUtils::FindPluginInMap(NewPlugins, PluginName);
		if (ensure(NewPlugin))
		{
			// Maintain behavior precedence for this function -- the new plugin doesn't superscede any previous versions (but we still add it for tracking)
			const bool bPluginAlreadyExists = KnownVersions && KnownVersions->Num() > 0;
			const DiscoveredPluginMapUtils::EInsertionType Priority = (bPluginAlreadyExists) ? DiscoveredPluginMapUtils::EInsertionType::AsSuppressedPlugin : DiscoveredPluginMapUtils::EInsertionType::AsOfferedPlugin;

			DiscoveredPluginMapUtils::InsertPluginIntoMap(AllPlugins, PluginName, *NewPlugin, Priority);

#if WITH_EDITOR
			if (Priority == DiscoveredPluginMapUtils::EInsertionType::AsOfferedPlugin)
			{
				AddToModuleNameToPluginMap(*NewPlugin);
			}
#endif //if WITH_EDITOR
		}

		return true;
	}
	else
	{
		FailReason = FText::Format(LOCTEXT("AddToPluginsListFailed", "Failed to load plugin '{0}'\n{1}"), FText::FromString(PluginFilename), FailReason);
		UE_LOGF(LogPluginManager, Error, "%ls", *FailReason.ToString());
		if (OutFailReason)
		{
			*OutFailReason = MoveTemp(FailReason);
		}
	}
#endif

	return false;
}

bool FPluginManager::RemoveFromPluginsList(const FString& PluginFilename, FText* OutFailReason /*= nullptr*/)
{
#if (WITH_ENGINE && !IS_PROGRAM) || WITH_PLUGIN_SUPPORT
	TSharedRef<FPlugin>* MaybePlugin = DiscoveredPluginMapUtils::FindPluginInMap_FromFileName(AllPlugins, PluginFilename);
	if (MaybePlugin == nullptr)
	{
		return true;
	}

	const TSharedRef<FPlugin> FoundPlugin = *MaybePlugin;

	FText FailReason;
	bool bSuccess = true;
	if (FoundPlugin->IsEnabled())
	{
		FailReason = FText::Format(LOCTEXT("PluginIsEnabled", "Plugin '{0}' is enabled"), FText::FromString(PluginFilename));
		bSuccess = false;
	}

	if(bSuccess)
	{
		bSuccess = VerifySinglePluginForAddOrRemove(FoundPlugin->Descriptor, FailReason);
	}

	if (!bSuccess)
	{
		if (OutFailReason)
		{
			*OutFailReason = FText::Format(LOCTEXT("RemoveFromPluginsListFailed", "Failed to remove plugin '{0}'\n{1}"), FText::FromString(PluginFilename), FailReason);
		}
		return false;
	}

#if WITH_EDITOR
	// Is this the plugin that would have been mapped?
	if (DiscoveredPluginMapUtils::IsOfferedPlugin(AllPlugins, *MaybePlugin))
	{
		RemoveFromModuleNameToPluginMap(FoundPlugin);
	}
#endif //if WITH_EDITOR

	const uint32 PluginNameHash = GetTypeHash(FoundPlugin->GetName());
	PluginsToConfigure.RemoveByHash(PluginNameHash, FoundPlugin->GetName());

	DiscoveredPluginMapUtils::RemovePluginFromMap(AllPlugins, FoundPlugin);
#endif //if (WITH_ENGINE && !IS_PROGRAM) || WITH_PLUGIN_SUPPORT

	return true;
}

void FPluginManager::DiscoverAllPlugins()
{
	ensure( AllPlugins.Num() == 0 );		// Should not have already been initialized!

	PluginSystemDefs::GetCmdLinePluginPaths(ExternalPluginSources);
	PluginSystemDefs::GetEnvPluginPaths(ExternalPluginSources);
	for (const FExternalPluginPath& ExternalSource : ExternalPluginSources)
	{
		PluginDiscoveryPaths.Add(ExternalSource.Path);
	}

	ReadAllPlugins(AllPlugins, PluginDiscoveryPaths);

	PluginsToConfigure.Reserve(AllPlugins.Num());
	for (const FDiscoveredPluginMap::ElementType& PluginPair : AllPlugins)
	{
		PluginsToConfigure.Add(PluginPair.Key);
#if WITH_EDITOR
		BuiltInPluginNames.Add(PluginPair.Key);
		AddToModuleNameToPluginMap(DiscoveredPluginMapUtils::ResolvePluginFromMapVal(PluginPair.Value));
#endif //if WITH_EDITOR
	}
}

void FPluginManager::ReadAllPlugins(FDiscoveredPluginMap& Plugins, const TSet<FString>& ExtraSearchPaths,
	TArray<FString>* OutPluginSources)
{
#if (WITH_ENGINE && !IS_PROGRAM) || (WITH_PLUGIN_SUPPORT && !UE_DISABLE_PLUGIN_DISCOVERY)
	TArray<FString> OptionalOutPluginRoots;
	const FProjectDescriptor* Project = IProjectManager::Get().GetCurrentProject();

#if !WITH_EDITOR
	// Find any plugin manifest files. These give us the plugin list (and their descriptors) without needing to scour the directory tree.
	TArray<FString> ManifestFileNames;
	if (Project != nullptr)
	{
		FindPluginManifestsInDirectory(*FPaths::ProjectPluginsDir(), ManifestFileNames);
	}
#endif // !WITH_EDITOR

	FScopedSlowTask SlowTask_ReadAll(3.f + (float)ExtraSearchPaths.Num());

	// track child plugins that don't want to go into main plugin set
	TArray<TSharedRef<FPlugin>> ChildPlugins;

#if !WITH_EDITOR
	// If we didn't find any manifests, do a recursive search for plugins
	if (ManifestFileNames.Num() == 0)
#endif
	{
		UE_LOGF(LogPluginManager, Verbose, "No *.upluginmanifest files found, looking for *.uplugin files instead.")

		SlowTask_ReadAll.EnterProgressFrame(0.5f);
		// Find "built-in" plugins.  That is, plugins situated right within the Engine directory.
		TArray<FString> EnginePluginDirs = FPaths::GetExtensionDirs(FPaths::EngineDir(), TEXT("Plugins"));
		if (OutPluginSources)
		{
			OptionalOutPluginRoots.Append(EnginePluginDirs);
		}
		{
			FScopedSlowTask SlowTask_ReadEngine((float)EnginePluginDirs.Num());

			for (const FString& EnginePluginDir : EnginePluginDirs)
			{
				SlowTask_ReadEngine.EnterProgressFrame();
				ReadPluginsInDirectory(EnginePluginDir, EPluginType::Engine, Plugins, ChildPlugins);
			}
		}

		SlowTask_ReadAll.EnterProgressFrame(0.5f);
		// Find plugins in the game project directory (<MyGameProject>/Plugins). If there are any engine plugins matching the name of a game plugin,
		// assume that the game plugin version is preferred.
		if (Project != nullptr)
		{
			TArray<FString> ProjectPluginDirs = FPaths::GetExtensionDirs(FPaths::GetPath(FPaths::GetProjectFilePath()), TEXT("Plugins"));
			if (OutPluginSources)
			{
				OptionalOutPluginRoots.Append(ProjectPluginDirs);
			}
			FScopedSlowTask SlowTask_ReadProject((float)ProjectPluginDirs.Num());

			for (const FString& ProjectPluginDir : ProjectPluginDirs)
			{
				SlowTask_ReadProject.EnterProgressFrame();
				ReadPluginsInDirectory(ProjectPluginDir, EPluginType::Project, Plugins, ChildPlugins);
			}
		}
	}
#if !WITH_EDITOR
	else
	{
		SlowTask_ReadAll.EnterProgressFrame();
		FScopedSlowTask SlowTask_ReadManifest((float)ManifestFileNames.Num());

		// Add plugins from each of the manifests
		for (const FString& ManifestFileName : ManifestFileNames)
		{
			SlowTask_ReadManifest.EnterProgressFrame();

			UE_LOGF(LogPluginManager, Verbose, "Reading plugin manifest: %ls", *ManifestFileName);
			if (OutPluginSources)
			{
				OutPluginSources->Add(FString::Printf(TEXT("Manifest: %s"),
					*FPaths::ConvertRelativePathToFull(ManifestFileName)));
			}
			FPluginManifest Manifest;

			// Try to load the manifest. We only expect manifests in a cooked game, so failing to load them is a hard error.
			FText FailReason;
			if (!Manifest.Load(ManifestFileName, FailReason))
			{
				UE_LOGF(LogPluginManager, Fatal, "%ls", *FailReason.ToString());
			}

			// Get all the standard plugin directories
			const FString EngineDir = FPaths::EngineDir();
			const FString EnterpriseDir = FPaths::EnterpriseDir();
			const FString ProjectModsDir = FPaths::ProjectModsDir();

			FScopedSlowTask SlowTask_ReadManifestContents((float)Manifest.Contents.Num());
			// Create all the plugins inside it
			for (const FPluginManifestEntry& Entry : Manifest.Contents)
			{
				SlowTask_ReadManifestContents.EnterProgressFrame();

				EPluginType Type;
				if (Entry.File.StartsWith(EngineDir))
				{
					Type = EPluginType::Engine;
				}
				else if (Entry.File.StartsWith(EnterpriseDir))
				{
					Type = EPluginType::Enterprise;
				}
				else if (Entry.File.StartsWith(ProjectModsDir))
				{
					Type = EPluginType::Mod;
				}
				else
				{
					Type = EPluginType::Project;
				}
				CreatePluginObject(Entry.File, Entry.Descriptor, Type, Plugins, ChildPlugins);
			}
		}
	}
#endif

	SlowTask_ReadAll.EnterProgressFrame();
	if (Project != nullptr)
	{
		FScopedSlowTask SlowTask_ReadProject(2.f + (float)Project->GetAdditionalPluginDirectories().Num());

		SlowTask_ReadProject.EnterProgressFrame();
		// Always add the mods from the loose directory without using manifests, because they're not packaged together.
		ReadPluginsInDirectory(FPaths::ProjectModsDir(), EPluginType::Mod, Plugins, ChildPlugins);
		if (OutPluginSources)
		{
			OptionalOutPluginRoots.Add(FPaths::ProjectModsDir());
		}

		// If they have a list of additional directories to check, add those plugins too
		for (const FString& Dir : Project->GetAdditionalPluginDirectories())
		{
			SlowTask_ReadProject.EnterProgressFrame();
			if (OutPluginSources)
			{
				OptionalOutPluginRoots.Add(Dir);
			}
			ReadPluginsInDirectory(Dir, EPluginType::External, Plugins, ChildPlugins);
		}

		SlowTask_ReadProject.EnterProgressFrame();
		// Add plugins from FPaths::EnterprisePluginsDir if it exists
		if (FPaths::DirectoryExists(FPaths::EnterprisePluginsDir()))
		{
			if (OutPluginSources)
			{
				OptionalOutPluginRoots.Add(FPaths::EnterprisePluginsDir());
			}
			ReadPluginsInDirectory(FPaths::EnterprisePluginsDir(), EPluginType::Enterprise, Plugins, ChildPlugins);
		}
	}

	for (const FString& ExtraSearchPath : ExtraSearchPaths)
	{
		SlowTask_ReadAll.EnterProgressFrame();
		if (OutPluginSources)
		{
			OptionalOutPluginRoots.Add(ExtraSearchPath);
		}
		ReadPluginsInDirectory(ExtraSearchPath, EPluginType::External, Plugins, ChildPlugins);
	}

	SlowTask_ReadAll.EnterProgressFrame();
	FScopedSlowTask SlowTask_ReadChildren((float)ChildPlugins.Num());

	// now that we have all the plugins, merge child plugins
	for (TSharedRef<FPlugin> Child : ChildPlugins)
	{
		SlowTask_ReadChildren.EnterProgressFrame();

		// find the parent
		TArray<FString> Tokens;
		FPaths::GetCleanFilename(Child->GetDescriptorFileName()).ParseIntoArray(Tokens, TEXT("_"), true);
		TSharedRef<FPlugin>* ParentPtr = nullptr;
		if (Tokens.Num() == 2)
		{
			FString ParentPluginName = Tokens[0];
			ParentPtr = DiscoveredPluginMapUtils::FindPluginInMap(Plugins, ParentPluginName);
		}
		if (ParentPtr != nullptr)
		{
			TSharedRef<FPlugin>& Parent = *ParentPtr;
			for (const FModuleDescriptor& ChildModule : Child->GetDescriptor().Modules)
			{
				bool bFound = false;

				// look for a matching parent
				for (FModuleDescriptor& ParentModule : Parent->Descriptor.Modules)
				{
					if (ParentModule.Name == ChildModule.Name && ParentModule.Type == ChildModule.Type)
					{
						bFound = true;
						// we only need to add the platform to an allow list if the parent had an allow list (otherwise, we could mistakenly remove all other platforms)
						if (ParentModule.bHasExplicitPlatforms || ParentModule.PlatformAllowList.Num() > 0)
						{
							ParentModule.PlatformAllowList.Append(ChildModule.PlatformAllowList);
						}

						// if we want to deny a platform, add it even if the parent didn't have a deny list. this won't cause problems with other platforms
						ParentModule.PlatformDenyList.Append(ChildModule.PlatformDenyList);

						// merge platform architecture allow/deny lists
						auto MergePlatformArchitectureLists = []( TMap<FString,TArray<FString>>& Parent, const TMap<FString,TArray<FString>>& Child )
						{
							for (TPair<FString,TArray<FString>> Pair : Child)
							{
								TArray<FString>* ArchList = Parent.Find(Pair.Key);
								if (ArchList == nullptr)
								{
									Parent.Add(Pair.Key, Pair.Value);
								}
								else
								{
									for (const FString& Arch : Pair.Value)
									{
										if (!ArchList->Contains(Arch))
										{
											ArchList->Add(Arch);
										}
									}
								}
							}
						};
						MergePlatformArchitectureLists( ParentModule.PlatformArchitectureAllowList, ChildModule.PlatformArchitectureAllowList);
						MergePlatformArchitectureLists( ParentModule.PlatformArchitectureDenyList, ChildModule.PlatformArchitectureDenyList);
					}
				}

				if (!bFound)
				{
					Parent->Descriptor.Modules.Add(ChildModule);
					UE_LOGF(LogPluginManager, Log, "Adding extension module %ls from %ls to %ls", *ChildModule.Name.ToString(), *Child->Name, *Parent->Name);
				}

			}

			if (Parent->GetDescriptor().bHasExplicitPlatforms || Parent->GetDescriptor().SupportedTargetPlatforms.Num() != 0)
			{
				for (const FString& SupportedTargetPlatform : Child->GetDescriptor().SupportedTargetPlatforms)
				{
					Parent->Descriptor.SupportedTargetPlatforms.AddUnique(SupportedTargetPlatform);
				}
			}

			// we need to remember the child's filename so we can make the file findable on the network file system
			Parent->PluginExtensionFileNameList.Add(Child->FileName);
		}
		else
		{
			UE_LOGF(LogPluginManager, Error, "Child plugin %ls was not named properly. It should be in the form <ParentPlugin>_<Platform>.uplugin.", *Child->GetDescriptorFileName());
		}
	}

	if (OutPluginSources)
	{
		for (const FString& Root : OptionalOutPluginRoots)
		{
			OutPluginSources->Add(FString::Printf(TEXT("PluginsDir: %s"),
				*FPaths::ConvertRelativePathToFull(Root)));
		}

		// Also report PakFiles, since ReadPluginsInDirectory searches PakFiles in addition to directories on disk
		if (Project)
		{
			bool bRunningWithPakFile =
				(FPlatformFileManager::Get().FindPlatformFile(TEXT("PakFile")) != nullptr);
			if (bRunningWithPakFile)
			{
				// TODO: Are there are pak files that might contain .uplugin files?
				OutPluginSources->Add(FString::Printf(TEXT("PakFiles: %s"),
					*FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectContentDir(), TEXT("Paks")))));
			}
		}
	}
#endif
}

void FPluginManager::ReadPluginsInDirectory(const FString& PluginsDirectory, const EPluginType Type, FDiscoveredPluginMap& Plugins, TArray<TSharedRef<FPlugin>>& ChildPlugins)
{
	// Make sure the directory even exists
	if(FPlatformFileManager::Get().GetPlatformFile().DirectoryExists(*PluginsDirectory))
	{
		FScopedSlowTask SlowTask_ReadPlugins(3.f);

		TArray<FString> FileNames;
		{
			SlowTask_ReadPlugins.EnterProgressFrame();
			FindPluginsInDirectory(PluginsDirectory, FileNames, FPlatformFileManager::Get().GetPlatformFile());
		}

		struct FLoadContext
		{
			FPluginDescriptor Descriptor;
			FText FailureReason;
			bool bResult = false;
		};

		TArray<FLoadContext> Contexts;
		Contexts.SetNum(FileNames.Num());
		{
			SlowTask_ReadPlugins.EnterProgressFrame();

			ParallelFor(TEXT("ReadPluginsInDirectory.PF"),
				FileNames.Num(),1,
				[&Contexts, &FileNames](int32 Index)
				{
					FLoadContext& Context = Contexts[Index];
					Context.bResult = Context.Descriptor.Load(FileNames[Index], Context.FailureReason);
				},
				EParallelForFlags::Unbalanced
			);
		}

		SlowTask_ReadPlugins.EnterProgressFrame();
		{
			FScopedSlowTask SlowTask_CreatePlugins((float)FileNames.Num());

			for (int32 Index = 0, Num = FileNames.Num(); Index < Num; ++Index)
			{
				SlowTask_CreatePlugins.EnterProgressFrame();

				const FString& FileName = FileNames[Index];
				FLoadContext& Context = Contexts[Index];

				if (Context.bResult)
				{
					CreatePluginObject(FileName, Context.Descriptor, Type, Plugins, ChildPlugins);
				}
				else
				{
					// NOTE: Even though loading of this plugin failed, we'll keep processing other plugins
					FString FullPath = FPaths::ConvertRelativePathToFull(FileName);
					UE_LOGF(LogPluginManager, Error, "Failed to load Plugin (%ls); %ls", *FullPath, *Context.FailureReason.ToString());
				}
			}
		}
	}
}

void FPluginManager::FindPluginsInDirectory(const FString& PluginsDirectory, TArray<FString>& FileNames, IPlatformFile& PlatformFile)
{
	//
	// Use our own custom file discovery method (over something like `IPlatformFile::IterateDirectoryRecursively()`)
	// to find all the .uplugin files. We utilize optimizations that `IterateDirectoryRecursively()` cannot, because
	// we know once we find one .uplugin file that there shouldn't be anymore in the same folder hierarchy.
	//

	// Directory visitor which aborts iteration once it finds a single .uplugin file
	class FFindPluginsInDirectory_Visitor : public IPlatformFile::FDirectoryVisitor
	{
	public:
		FFindPluginsInDirectory_Visitor(TArray<FString>& OutSubDirectories)
			: IPlatformFile::FDirectoryVisitor(EDirectoryVisitorFlags::ThreadSafe)
			, SubDirectories(OutSubDirectories)
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			int32 FileNameLen = FCString::Strlen(FilenameOrDirectory);

			if (bIsDirectory)
			{
				constexpr const TCHAR ContentStr[] = TEXT("/Content");
				constexpr const int32 ContentStrLen = UE_ARRAY_COUNT(ContentStr)-1; // -1 for the string's terminating zero (which would be counted by `UE_ARRAY_COUNT()`)

				if (FileNameLen >= ContentStrLen && FCString::Stricmp(&FilenameOrDirectory[FileNameLen - ContentStrLen], ContentStr) == 0)
				{
					// excluded
				}
				else
				{
					SubDirectories.Emplace(FilenameOrDirectory);
				}
			}
			else
			{
				// ideally we'd like to find this .uplugin file first before visiting anything else
				constexpr const TCHAR PluginExt[] = TEXT(".uplugin");
				constexpr const int32 PluginExtLen = UE_ARRAY_COUNT(PluginExt)-1; // -1 for the string's terminating zero (which would be counted by `UE_ARRAY_COUNT()`)

				if (FileNameLen >= PluginExtLen && FCString::Strcmp(&FilenameOrDirectory[FileNameLen - PluginExtLen], PluginExt) == 0)
				{
					FoundPluginFile = FilenameOrDirectory;

					// Stop the iteration, we've found the .uplugin we're looking for
					return false;
				}
			}
			return true;
		}

		FString FoundPluginFile;
		TArray<FString>& SubDirectories;
	};

	TArray<FString> DirectoriesToVisit;
	DirectoriesToVisit.Add(PluginsDirectory);

	FScopedSlowTask SlowTask(1000.f); // Pick an arbitrary amount of work that is resiliant to some floating point multiplication & division

	constexpr int32 MinBatchSize = 1;
	TArray<TArray<FString>> DirectoriesToVisitNext;
	FRWLock FoundFilesLock;
	while (DirectoriesToVisit.Num() > 0)
	{
		const float TotalWorkRemaining = SlowTask.TotalAmountOfWork - SlowTask.CompletedWork - SlowTask.CurrentFrameScope;
		SlowTask.EnterProgressFrame(TotalWorkRemaining);
		const int32 UnitsOfWorkTodoThisLoop = DirectoriesToVisit.Num();

		ParallelForWithTaskContext(TEXT("FindPluginsInDirectory.PF"),
			DirectoriesToVisitNext,
			DirectoriesToVisit.Num(),
			MinBatchSize,
			[&FoundFilesLock, &FileNames, &DirectoriesToVisit, &PlatformFile](TArray<FString>& OutDirectoriesToVisitNext, int32 Index)
			{
				// Track where we start pushing sub-directories to because we might want to discard them (if we end up finding a .uplugin).
				// Because of how `ParallelForWithTaskContext()` works, this array may already be populated from another execution,
				// so we have to be targeted about what we clear from the array.
				const int32 StartingDirIndex = OutDirectoriesToVisitNext.Num();

				FFindPluginsInDirectory_Visitor Visitor(OutDirectoriesToVisitNext); // This visitor writes directly to `OutDirectoriesToVisitNext`, which is why we have to manage its contents
				PlatformFile.IterateDirectory(*DirectoriesToVisit[Index], Visitor);
				if (!Visitor.FoundPluginFile.IsEmpty())
				{
					// Since we found a .uplugin, ignore sub-directories (stop from iterating deeper) --
					// there shouldn't be any other .uplugin files deeper.
					// Also, disallow shrinking -- because we're trying to be fast and would rather skip mem reallocs.
					OutDirectoriesToVisitNext.RemoveAt(StartingDirIndex, OutDirectoriesToVisitNext.Num() - StartingDirIndex, EAllowShrinking::No);

					// Multiple tasks may be trying to write to this at the same time, lock it
					FRWScopeLock ScopeLock(FoundFilesLock, SLT_Write);
					FileNames.Emplace(MoveTemp(Visitor.FoundPluginFile));
				}
			},
			EParallelForFlags::Unbalanced);

		// Adjust the scope of work done this frame, since we discovered more work
		const int32 NewKnownUnitsOfWork = UnitsOfWorkTodoThisLoop + DirectoriesToVisitNext.Num();
		SlowTask.CurrentFrameScope = (float)UnitsOfWorkTodoThisLoop * (TotalWorkRemaining / (float)NewKnownUnitsOfWork);

		// Clear and resize `DirectoriesToVisit` for the next batch.
		DirectoriesToVisit.Reset(Algo::TransformAccumulate(DirectoriesToVisitNext, &TArray<FString>::Num, 0));
		// Copy all the `DirectoriesToVisitNext` (populated by the various `ParallelFor` tasks) into the one array we use to feed the next round of tasks.
		for (TArray<FString>& Directories : DirectoriesToVisitNext)
		{
			DirectoriesToVisit.Append(MoveTemp(Directories));
		}
	}
}

void FPluginManager::FindPluginManifestsInDirectory(const FString& PluginManifestDirectory, TArray<FString>& FileNames)
{
	class FManifestVisitor : public IPlatformFile::FDirectoryVisitor
	{
	public:
		TArray<FString>& ManifestFileNames;

		FManifestVisitor(TArray<FString>& InManifestFileNames) : ManifestFileNames(InManifestFileNames)
		{
		}

		virtual bool Visit(const TCHAR* FileNameOrDirectory, bool bIsDirectory)
		{
			if (!bIsDirectory)
			{
				FStringView FileName(FileNameOrDirectory);
				if (FileName.EndsWith(TEXT(".upluginmanifest")))
				{
					ManifestFileNames.Emplace(FileName);
				}
			}
			return true;
		}
	};

	FManifestVisitor Visitor(FileNames);
	IFileManager::Get().IterateDirectory(*PluginManifestDirectory, Visitor);
}

void FPluginManager::CreatePluginObject(const FString& FileName, const FPluginDescriptor& Descriptor, const EPluginType Type, FDiscoveredPluginMap& Plugins, TArray<TSharedRef<FPlugin>>& ChildPlugins)
{
	TSharedRef<FPlugin> Plugin = MakeShareable(new FPlugin(FileName, Descriptor, Type));

	// children plugins are gathered and used later
	if (Plugin->GetDescriptor().bIsPluginExtension)
	{
		ChildPlugins.Add(Plugin);
		return;
	}

	FString FullPath = FPaths::ConvertRelativePathToFull(FileName);
	UE_LOGF(LogPluginManager, Verbose, "Read plugin descriptor for %ls, from %ls", *Plugin->GetName(), *FullPath);

	TSharedRef<FPlugin>* ExistingPlugin = DiscoveredPluginMapUtils::FindPluginInMap(Plugins, Plugin->GetName());
	if (ExistingPlugin == nullptr)
	{
		DiscoveredPluginMapUtils::InsertPluginIntoMap(Plugins, Plugin->GetName(), Plugin);
	}
	// We allow for duplicates of plugins between engine and the project, favoring the project level plugin
	else if ((*ExistingPlugin)->Type == EPluginType::Engine && Type == EPluginType::Project && !(*ExistingPlugin)->bEnabled)
	{
		UE_LOGF(LogPluginManager, Display, "By default, prioritizing project plugin (%ls) over the corresponding engine version (%ls).", *Plugin->FileName, *(*ExistingPlugin)->FileName);
		DiscoveredPluginMapUtils::InsertPluginIntoMap(Plugins, Plugin->GetName(), Plugin, DiscoveredPluginMapUtils::EInsertionType::AsOfferedPlugin);
	}
	else if ((*ExistingPlugin)->FileName != Plugin->FileName)
	{
		if ((*ExistingPlugin)->bEnabled)
		{
			UE_LOGF(LogPluginManager, Display, "A version of the '%ls' plugin has already been enabled (%ls); prioritizing that over the newly discovered version (%ls).", *Plugin->Name, *(*ExistingPlugin)->FileName, *Plugin->FileName);
			DiscoveredPluginMapUtils::InsertPluginIntoMap(Plugins, Plugin->GetName(), Plugin, DiscoveredPluginMapUtils::EInsertionType::AsSuppressedPlugin);
		}
		else if ((*ExistingPlugin)->Type == EPluginType::Project && Type == EPluginType::Engine)
		{
			// Project plugins are favored over engine plugins, so we don't want to warn in this case
			// (instead we mimic the Verbose log from above, and just explain which plugin we're favoring)
			UE_LOGF(LogPluginManager, Display, "By default, prioritizing project plugin (%ls) over the corresponding engine version (%ls).", *(*ExistingPlugin)->FileName, *Plugin->FileName);
			DiscoveredPluginMapUtils::InsertPluginIntoMap(Plugins, Plugin->GetName(), Plugin, DiscoveredPluginMapUtils::EInsertionType::AsSuppressedPlugin);
		}
		else
		{
			const int32 ExistingVersion = (*ExistingPlugin)->GetDescriptor().Version;
			if (ExistingVersion != Descriptor.Version)
			{
				UE_LOGF(
					LogPluginManager,
					Display,
					"By default, prioritizing newer version (v%d) of '%ls' plugin, over older version (v%d).",
					FMath::Max(ExistingVersion, Descriptor.Version),
					*Plugin->GetName(),
					FMath::Min(ExistingVersion, Descriptor.Version));

				if (ExistingVersion < Descriptor.Version)
				{
					DiscoveredPluginMapUtils::InsertPluginIntoMap(Plugins, Plugin->GetName(), Plugin, DiscoveredPluginMapUtils::EInsertionType::AsOfferedPlugin);
				}
				else
				{
					DiscoveredPluginMapUtils::InsertPluginIntoMap(Plugins, Plugin->GetName(), Plugin, DiscoveredPluginMapUtils::EInsertionType::AsSuppressedPlugin);
				}
			}
			else
			{
				UE_LOGF(LogPluginManager, Warning, "The same version (v%d) of plugin '%ls' exists at '%ls' and '%ls' - second location will be ignored.", ExistingVersion, *Plugin->GetName(), *(*ExistingPlugin)->FileName, *Plugin->FileName);
			}
		}
	}
	// else, same plugin? Do nothing -- keep from adding it twice.
}

// Helper class to find all pak files.
class FPakFileSearchVisitor : public IPlatformFile::FDirectoryVisitor
{
	TArray<FString>& FoundFiles;
public:
	FPakFileSearchVisitor(TArray<FString>& InFoundFiles)
		: FoundFiles(InFoundFiles)
	{}
	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
	{
		if (bIsDirectory == false)
		{
			FString Filename(FilenameOrDirectory);
			if (Filename.MatchesWildcard(TEXT("*.pak")) && !FoundFiles.Contains(Filename))
			{
				FoundFiles.Add(Filename);
			}
		}
		return true;
	}
};

void FPluginManager::SetBinariesRootDirectories(const FString& InEngineBinariesRootDir, const FString& InProjectBinariesRootDir)
{
	EngineBinariesRootDir = InEngineBinariesRootDir;
	ProjectBinariesRootDir = InProjectBinariesRootDir;
}

void FPluginManager::SetPreloadBinaries()
{
	bPreloadedBinaries = true;
}

bool FPluginManager::GetPreloadBinaries()
{
	return bPreloadedBinaries;
}

FPluginManager::FLeakedPackagesFoundEvent& FPluginManager::OnLeakedPackagesFound()
{
	return LeakedPackagesFoundEvent;
}


#if READ_TARGET_ENABLED_PLUGINS_FROM_RECEIPT
static TUniquePtr<FTargetReceipt> FindMatchingTargetFile (const FString& ReceiptWildcard)
{
	FString DefaultEditorTarget;
	GConfig->GetString(TEXT("/Script/BuildSettings.BuildSettings"), TEXT("DefaultEditorTarget"), DefaultEditorTarget, GEngineIni);

	TArray<FString> AllTargetFilesWithoutPath;
	const FString ReceiptPath = FPaths::GetPath(ReceiptWildcard);
	IFileManager::Get().FindFiles(AllTargetFilesWithoutPath, *ReceiptWildcard, true, false);

	// Used as a fallback in lieu of an exact match (only for EBuildTargetType::Editor).
	TPair<FString, TUniquePtr<FTargetReceipt>> BestMatch;

	for (const FString& TargetFileWithoutPath : AllTargetFilesWithoutPath)
	{
		FString TargetFile = FPaths::Combine(ReceiptPath, TargetFileWithoutPath);
		TUniquePtr<FTargetReceipt> Receipt = MakeUnique<FTargetReceipt>();
		if (Receipt->Read(TargetFile))
		{
			if (Receipt->TargetType == FApp::GetBuildTargetType() && Receipt->Configuration == FApp::GetBuildConfiguration())
			{
				// Priority:
				// - If we are non-editor, first non-editor receipt
				// - Otherwise, first that matches the running executable
				// - Otherwise, first corresponding to DefaultEditorTarget (if specified)
				// - Otherwise, first editor receipt
				if (Receipt->TargetType != EBuildTargetType::Editor || Receipt->LaunchesCurrentExecutable())
				{
					UE_LOGF(LogPluginManager, Log, "Found matching target receipt: %ls", *TargetFile);
					return Receipt;
				}

				const bool bIsDefaultTarget = (DefaultEditorTarget.Len() != 0) && (DefaultEditorTarget == Receipt->TargetName);
				if (bIsDefaultTarget || !BestMatch.Value)
				{
					BestMatch = { MoveTemp(TargetFile), MoveTemp(Receipt) };
					// `TargetFile` and `Receipt` are now invalidated by virtue of being moved-from.
				}
			}
		}
	}

	if (BestMatch.Value)
	{
		UE_LOGF(LogPluginManager, Log, "Found target receipt: %ls", *BestMatch.Key);
	}
	else
	{
		UE_LOGF(LogPluginManager, Log, "Unable to find target receipt in path: %ls", *ReceiptWildcard);
	}

	// Might be null.
	return MoveTemp(BestMatch.Value);
}
#endif // READ_TARGET_ENABLED_PLUGINS_FROM_RECEIPT

void FPluginManager::FindCompileTimePlugins(FPluginDiscoveryContext& Context)
{
#if !UE_BUILD_SHIPPING
#if READ_TARGET_ENABLED_PLUGINS_FROM_RECEIPT
	{
		SCOPED_BOOT_TIMING("ReadTargetBuildPluginsFromReceipt");

		// Read the build plugins from the target file using the target receipt file. This controls which optional plugin references can be enabled.
		auto ReadBuildPluginsFromFirstMatchingTargetFile = [&Context](const TCHAR* BaseDir) -> bool
			{
				const FString ReceiptWildcard = FTargetReceipt::GetDefaultPath(BaseDir, TEXT("*"), FPlatformProcess::GetBinariesSubdirectory(), FApp::GetBuildConfiguration(), nullptr);

				TUniquePtr<FTargetReceipt> Receipt = FindMatchingTargetFile(ReceiptWildcard);
				if (Receipt.IsValid())
				{
					Context.AllowedOptionalDependencies.Append(Receipt->BuildPlugins);
					return true;
				}
				return false;
			};

		UE_LOGF(LogPluginManager, Log, "Looking for build plugins target receipt");
		if (!ReadBuildPluginsFromFirstMatchingTargetFile(FPlatformMisc::ProjectDir()))
		{
			ReadBuildPluginsFromFirstMatchingTargetFile(FPlatformMisc::EngineDir());
		}
	}
#else
	{
		// Configure the plugins that were enabled from the target file using defines
		SCOPED_BOOT_TIMING("ReadTargetBuildPlugins");
		Context.AllowedOptionalDependencies.Append({ UBT_TARGET_BUILD_PLUGINS });
	}
#endif // READ_TARGET_ENABLED_PLUGINS_FROM_RECEIPT
#endif
}

// Check which plugins have been enabled or excluded via the command line
bool FPluginManager::FindCommandLinePlugins(FPluginDiscoveryContext& Context)
{
	SCOPED_BOOT_TIMING("ParseCmdLineForPlugins");

	auto ParsePluginsList = [this](const TCHAR* InListKey) -> TArray<FString>
		{
			TArray<FString> PluginsList;
			{
				FString PluginsListStr;
				FParse::Value(FCommandLine::Get(), InListKey, PluginsListStr, false);
				PluginsListStr.ParseIntoArray(PluginsList, TEXT(","));
			}

			TArray<FString> PluginListFilenames;
			PluginsList.RemoveAll([&PluginListFilenames](const FString& ExceptPluginName)
				{
					if (ExceptPluginName.EndsWith(TEXT(".txt")))
					{
						PluginListFilenames.Add(ExceptPluginName);
						return true;
					}
					return false;
				});

			for (const FString& PluginListFilename : PluginListFilenames) //-V1078
			{
				FFileHelper::LoadFileToStringWithLineVisitor(*PluginListFilename, [&PluginsList](FStringView Line)
					{
						PluginsList.Add(FString(Line));
					});
			}

			TArray<FString> WildcardPluginsList;
			PluginsList.RemoveAll([&WildcardPluginsList](const FString& ExceptPluginName)
				{
					constexpr FAsciiSet Wildcards("*?");
					if (FAsciiSet::HasAny(ExceptPluginName, Wildcards))
					{
						WildcardPluginsList.Add(ExceptPluginName);
						return true;
					}
					return false;
				});

			if (WildcardPluginsList.Num() > 0)
			{
				for (const FString& PotentialPluginName : PluginsToConfigure)
				{
					bool bMatchesAnyWildcard = false;
					for (const FString& WildcardPluginName : WildcardPluginsList)
					{
						if (PotentialPluginName.MatchesWildcard(WildcardPluginName))
						{
							bMatchesAnyWildcard = true;
							break;
						}
					}

					if (bMatchesAnyWildcard)
					{
						PluginsList.Add(PotentialPluginName);
					}
				}
			}

			return PluginsList;
		};

	// Which extra plugins should be enabled?
	bAllPluginsEnabledViaCommandLine = FParse::Param(FCommandLine::Get(), TEXT("EnableAllPlugins"));
	TArray<FString> ExtraPluginsToEnable;
	if (bAllPluginsEnabledViaCommandLine)
	{
		ExtraPluginsToEnable = PluginsToConfigure.Array();
	}
	else
	{
		ExtraPluginsToEnable = ParsePluginsList(TEXT("EnablePlugins="));
	}
	if (ExtraPluginsToEnable.Num() > 0)
	{
#if !UE_BUILD_SHIPPING
		Context.AllowedOptionalDependencies.Append(ExtraPluginsToEnable);
#endif

		auto IsRestrictedPlugin = [this](const FString& PluginName)
			{
				if (TSharedPtr<IPlugin> PluginPtr = FindPlugin(PluginName))
				{
					const FString& PluginBaseDir = PluginPtr->GetBaseDir();
					return PluginBaseDir.Contains(TEXT("/Restricted/"));
				}
				return true;
			};

		const TArray<FString> ExceptPlugins = ParsePluginsList(TEXT("ExceptPlugins="));
		const bool bExceptRestrictedPlugins = FParse::Param(FCommandLine::Get(), TEXT("ExceptRestrictedPlugins"));
		for (const FString& EnablePluginName : ExtraPluginsToEnable)
		{
			if (!Context.ConfiguredPluginNames.Contains(EnablePluginName) && !ExceptPlugins.Contains(EnablePluginName) && (!bExceptRestrictedPlugins || !IsRestrictedPlugin(EnablePluginName)))
			{
				if (!ConfigureEnabledPluginForCurrentTarget(FPluginReferenceDescriptor(EnablePluginName, true), Context.EnabledPlugins,
					bAllPluginsEnabledViaCommandLine ? TEXTVIEW("Commandline EnableAllPlugins") : TEXTVIEW("CommandLine EnablePlugins="), Context.AllowedOptionalDependencies))
				{
					if (bAllPluginsEnabledViaCommandLine)
					{
						// Plugins may legitimately fail to enable when running with -EnableAllPlugins, but this shouldn't be considered a fatal error
						continue;
					}
					return false;
				}
				Context.ConfiguredPluginNames.Add(EnablePluginName);
			}
		}
	}

	// Which extra plugins should be disabled?
	TArray<FString> ExtraPluginsToDisable;
	ExtraPluginsToDisable = ParsePluginsList(TEXT("DisablePlugins="));
	for (const FString& DisablePluginName : ExtraPluginsToDisable)
	{
		if (!Context.ConfiguredPluginNames.Contains(DisablePluginName))
		{
			if (!ConfigureEnabledPluginForCurrentTarget(FPluginReferenceDescriptor(DisablePluginName, false), Context.EnabledPlugins,
				TEXTVIEW("CommandLine DisablePlugins="), Context.AllowedOptionalDependencies))
			{
				return false;
			}
			Context.ConfiguredPluginNames.Add(DisablePluginName);
		}
	}

	return true;
}

bool FPluginManager::FindTargetPlugins(FPluginDiscoveryContext& Context)
{
	// this is a poorly named param, and probably could be just removed (this code enables engine and project plugins)
	if (!FParse::Param(FCommandLine::Get(), TEXT("NoEnginePlugins")))
	{
		SCOPED_BOOT_TIMING("EnginePlugins");

		// May be overridden later by receipt or project descriptor.
		bool bAllowEnginePluginsEnabledByDefault = true;

#if READ_TARGET_ENABLED_PLUGINS_FROM_RECEIPT
		// Configure the plugins that were enabled or disabled from the target file using the target receipt file
		auto ConfigurePluginsFromFirstMatchingTargetFile = [this, &Context, &bAllowEnginePluginsEnabledByDefault]
			(const TCHAR* BaseDir, bool& bOutError) -> bool
			{
				const FString ReceiptWildcard = FTargetReceipt::GetDefaultPath(BaseDir, TEXT("*"), FPlatformProcess::GetBinariesSubdirectory(), FApp::GetBuildConfiguration(), nullptr);
				FString SourceDescription = FString::Printf(TEXT("Receipt files %s"), *ReceiptWildcard);

				TUniquePtr<FTargetReceipt> Receipt = FindMatchingTargetFile(ReceiptWildcard);
				if (Receipt.IsValid())
				{
					FReceiptProperty* AllowDefaultProperty = Algo::FindByPredicate(Receipt->AdditionalProperties,
						[](const FReceiptProperty& InProp) -> bool { return InProp.Name == TEXT("bAllowEnginePluginsEnabledByDefault"); });
					if (AllowDefaultProperty && !FCString::Stricmp(*AllowDefaultProperty->Value, TEXT("false")))
					{
						bAllowEnginePluginsEnabledByDefault = false;
					}

					for (const TPair<FString, bool>& Pair : Receipt->PluginNameToEnabledState)
					{
						const FString& PluginName = Pair.Key;
						const bool bEnabled = Pair.Value;
						if (!Context.ConfiguredPluginNames.Contains(PluginName))
						{
							if (!ConfigureEnabledPluginForCurrentTarget(FPluginReferenceDescriptor(PluginName, bEnabled), Context.EnabledPlugins,
								SourceDescription, Context.AllowedOptionalDependencies))
							{
								bOutError = true;
								break;
							}
							Context.ConfiguredPluginNames.Add(PluginName);
						}
					}

					return true;
				}

				return false;
			};

		{
			SCOPED_BOOT_TIMING("ConfigureTargetEnabledPluginsFromReceipt");
			UE_LOGF(LogPluginManager, Log, "Looking for enabled plugins target receipt");
			bool bErrorConfiguring = false;
			if (!ConfigurePluginsFromFirstMatchingTargetFile(FPlatformMisc::ProjectDir(), bErrorConfiguring))
			{
				ConfigurePluginsFromFirstMatchingTargetFile(FPlatformMisc::EngineDir(), bErrorConfiguring);
			}
			if (bErrorConfiguring)
			{
				return false;
			}
		}
#else
		{
			// Configure the plugins that were enabled from the target file using defines
			SCOPED_BOOT_TIMING("ConfigureTargetEnabledPlugins");

			for (const FStringView& TargetEnabledPlugin : std::initializer_list<FStringView>{UBT_TARGET_ENABLED_PLUGINS})
			{
				if (!Context.ConfiguredPluginNames.ContainsByHash(GetTypeHash(TargetEnabledPlugin), TargetEnabledPlugin))
				{
					if (!ConfigureEnabledPluginForCurrentTarget(FPluginReferenceDescriptor(FString{TargetEnabledPlugin}, true), Context.EnabledPlugins,
						TEXTVIEW("UBT_TARGET_ENABLED_PLUGINS"), Context.AllowedOptionalDependencies))
					{
						return false;
					}
					Context.ConfiguredPluginNames.Add(FString{TargetEnabledPlugin});
				}
			}
		}
		{
			// Configure the plugins that were disabled from the target file using defines
			SCOPED_BOOT_TIMING("ConfigureTargetDisabledPlugins");

			for (const FStringView& TargetDisabledPlugin : std::initializer_list<FStringView>{UBT_TARGET_DISABLED_PLUGINS})
			{
				if (!Context.ConfiguredPluginNames.ContainsByHash(GetTypeHash(TargetDisabledPlugin), TargetDisabledPlugin))
				{
					if (!ConfigureEnabledPluginForCurrentTarget(FPluginReferenceDescriptor(FString{TargetDisabledPlugin}, false), Context.EnabledPlugins,
						TEXTVIEW("UBT_TARGET_ENABLED_PLUGINS"), Context.AllowedOptionalDependencies))
					{
						return false;
					}
					Context.ConfiguredPluginNames.Add(FString{TargetDisabledPlugin});
				}
			}
		}
#endif // READ_TARGET_ENABLED_PLUGINS_FROM_RECEIPT

		auto ProcessPluginConfigurations = [&Context, this]
			(const TArray<FPluginReferenceDescriptor>& PluginReferences, FStringView SourceDescription)->bool
			{
				for (const FPluginReferenceDescriptor& PluginReference : PluginReferences)
				{
					if (!Context.ConfiguredPluginNames.Contains(PluginReference.Name))
					{
						if (!ConfigureEnabledPluginForCurrentTarget(PluginReference, Context.EnabledPlugins, SourceDescription, Context.AllowedOptionalDependencies))
						{
							return false;
						}
						Context.ConfiguredPluginNames.Add(PluginReference.Name);
					}
				}
				return true;
			};

		// Find all the plugin references in the project file
		const FProjectDescriptor* ProjectDescriptor = IProjectManager::Get().GetCurrentProject();
		{
			SCOPED_BOOT_TIMING("AddPluginReferences");

			if (ProjectDescriptor != nullptr)
			{
				bAllowEnginePluginsEnabledByDefault = !ProjectDescriptor->bDisableEnginePluginsByDefault;

				// Copy the plugin references, since we may modify the project if any plugins are missing
				TArray<FPluginReferenceDescriptor> PluginReferences(ProjectDescriptor->Plugins);
				if (!ProcessPluginConfigurations(PluginReferences,
					FString::Printf(TEXT("Enabled plugins in .uproject for %s"), *ProjectDescriptor->Description)))
				{
					return false;
				}
			}
		}

		{
			// Add the plugins which are enabled by default
			SCOPED_BOOT_TIMING("AddPluginsEnabledByDefault");

			FString SourceDescription = FString::Printf(TEXT(".uplugin files that are enabled by default"));
			for (const FString& PluginName : PluginsToConfigure)
			{
				const TSharedRef<FPlugin>& Plugin = DiscoveredPluginMapUtils::FindPluginInMap_Checked(AllPlugins, PluginName);
				if (Plugin->IsEnabledByDefault(bAllowEnginePluginsEnabledByDefault) && !Context.ConfiguredPluginNames.Contains(PluginName))
				{
					if (!ConfigureEnabledPluginForCurrentTarget(FPluginReferenceDescriptor(PluginName, true),
						Context.EnabledPlugins, SourceDescription, Context.AllowedOptionalDependencies))
					{
						return false;
					}
					Context.ConfiguredPluginNames.Add(PluginName);
				}
			}
		}
	}

	return true;
}


bool FPluginManager::FindProgramPlugins(FPluginDiscoveryContext& Context)
{
#if IS_PROGRAM
	// Programs can also define the list of enabled plugins in ini
	SCOPED_BOOT_TIMING("AddProgramEnabledPlugins");

	auto EnableProgramPlugin = [this, &Context](const TCHAR* ConfigEntry, bool bOptionalPlugin) mutable
		{
			TArray<FString> ProgramPluginNames;
			GConfig->GetArray(TEXT("Plugins"), ConfigEntry, ProgramPluginNames, GEngineIni);
			FString SourceDescription = FString::Printf(TEXT("Engine.ini:[%s]:ProgramEnabledPlugins"), ConfigEntry);

			for (const FString& PluginName : ProgramPluginNames)
			{
				if (!Context.ConfiguredPluginNames.Contains(PluginName))
				{
					FPluginReferenceDescriptor PluginReference(PluginName, true);
					PluginReference.bOptional = bOptionalPlugin;
					if (!ConfigureEnabledPluginForCurrentTarget(PluginReference,
						Context.EnabledPlugins, SourceDescription, Context.AllowedOptionalDependencies))
					{
						return false;
					}
					Context.ConfiguredPluginNames.Add(PluginName);
				}
			}

			return true;
		};

	if (!EnableProgramPlugin(TEXT("ProgramEnabledPlugins"), false) || !EnableProgramPlugin(TEXT("ProgramOptionalPlugins"), true))
	{
		return false;
	}
#endif
	return true;
}

void FPluginManager::MarkEnabledPlugins(FPluginDiscoveryContext& Context)
{
	// Mark all the plugins as enabled
	SCOPED_BOOT_TIMING("MarkEnabledPlugins");

	// Merged modular builds act as if they were monolithic as far as plugin discovery is concerned.
#if !IS_MONOLITHIC && !UE_MERGED_MODULES
	TMap<FString, FString> PluginToPhysicalFile;

	// If EngineBinariesRootDir is set it means that we are loading binaries from somewhere else than our basedir.
	// We need to search for all the plugins under binaries root dir in order to find dlls etc.
	// TODO: Maybe plugin desc inside pak files should store original relative path?
	if (!EngineBinariesRootDir.IsEmpty())
	{
		TArray<FString> PhysicalFileNames;

		for (const FString& EnginePluginDir : FPaths::GetExtensionDirs(EngineBinariesRootDir, TEXT("Plugins"), !GIsEditor))
		{
			FindPluginsInDirectory(EnginePluginDir, PhysicalFileNames, IPlatformFile::GetPlatformPhysical());
		}

		for (const FString& ProjectPluginDir : FPaths::GetExtensionDirs(ProjectBinariesRootDir, TEXT("Plugins"), !GIsEditor))
		{
			FindPluginsInDirectory(ProjectPluginDir, PhysicalFileNames, IPlatformFile::GetPlatformPhysical());
		}

		for (const FString& FileName : PhysicalFileNames)
		{
			FString PluginName = FPaths::GetBaseFilename(FileName);
			PluginToPhysicalFile.Add(PluginName, FileName);
		}
	}
#endif // !IS_MONOLITHIC && !UE_MERGED_MODULES

	for (TPair<FString, FPlugin*>& Pair : Context.EnabledPlugins)
	{
		FPlugin& Plugin = *Pair.Value;

#if !IS_MONOLITHIC && !UE_MERGED_MODULES
		// Mount the binaries directory, and check the modules are valid.
		// We do not run this process while using merged modular build, we'll keep acting as if the build was monolithic.
		if (Plugin.Descriptor.Modules.Num() > 0)
		{
			const TCHAR* PluginFile = *Plugin.FileName;

			// If we have a overridden binary path, use that instead
			if (FString* PhysicalFile = PluginToPhysicalFile.Find(Plugin.Name))
			{
				PluginFile = **PhysicalFile;
			}

			// Mount the binaries directory
			const FString PluginBinariesPath = FPaths::Combine(FPaths::GetPath(PluginFile), TEXT("Binaries"), FPlatformProcess::GetBinariesSubdirectory());
			FModuleManager::Get().AddBinariesDirectory(*PluginBinariesPath, Plugin.GetLoadedFrom() == EPluginLoadedFrom::Project);
		}

		// Check the declared engine version. This is a soft requirement, so allow the user to skip over it.
		if (!IsPluginCompatible(Plugin) && !PromptToLoadIncompatiblePlugin(Plugin))
		{
			UE_LOGF(LogPluginManager, Display, "Skipping load of '%ls'.", *Plugin.Name);
			continue;
		}
#endif // !IS_MONOLITHIC && !UE_MERGED_MODULES
		if (!Plugin.GetDescriptor().bExplicitlyLoaded && Plugin.GetDescriptor().LocalizationTargets.Num() > 0)
		{
			PluginLocalizationUtils::GetLocalizationPathsForPlugin(Plugin, Context.AdditionalLocResPaths);
		}

		Plugin.bEnabled = true;
	}
}

void FPluginManager::ProcessEnabledPlugins(FPluginDiscoveryContext& Context)
{
	SCOPED_BOOT_TIMING("ProcessEnabledPlugins");

	PluginsEnabledForTarget.Reserve(Context.EnabledPlugins.Num());
	for (const TPair<FString, FPlugin*>& Pair : Context.EnabledPlugins)
	{
		if (Pair.Value->bEnabled)
		{
			PluginsEnabledForTarget.Add(Pair.Key);
		}
	}
	// Initially this will be correct. PluginsEnabledForTarget will be updated with new mounts and this will stop including compiled plugins.
	PluginsEnabledAtStartup.Append(PluginsEnabledForTarget);


	// generate optimal list of plugins to process
	TArray<TSharedRef<FPlugin>> PluginsArray;
	for (const FString& PluginName : PluginsToConfigure)
	{
		TSharedRef<FPlugin> Plugin = DiscoveredPluginMapUtils::FindPluginInMap_Checked(AllPlugins, PluginName);

		// only process enabled plugins
		if (Plugin->bEnabled && !Plugin->Descriptor.bExplicitlyLoaded)
		{
			PluginsArray.Add(Plugin);
		}
	}

	TArray<FString> ConfigFilesPluginsCannotOverride;
	GConfig->GetArray(TEXT("Plugins"), TEXT("ConfigFilesPluginsCannotOverride"), ConfigFilesPluginsCannotOverride, GEngineIni);

	FCriticalSection ConfigCS;
	FCriticalSection PluginPakCS;

	// Mount all the enabled plugins
	FConfigModificationTracker ChangeTracker;
	ChangeTracker.bTrackLoadedFiles = true;
	ChangeTracker.bSkipLoadingAlreadyLoadedFiles = true;

	TArray<FName> PluginNames;
	PluginNames.Reserve(PluginsArray.Num());

	// walk over each plugin, and add the config files that are named for the plugin
	// this is a separate loop so that a plugin can modify another plugin's configs in the second loop below
	for (TSharedRef<FPlugin> PluginPtr : PluginsArray)
	{
		FPlugin& Plugin = *PluginPtr;
		FName PluginName(*Plugin.Name);
		PluginNames.Add(PluginName);

		UE_LOGF(LogPluginManager, Log, "Mounting %ls plugin %ls", *EnumToString(Plugin.Type), *Plugin.GetName());
		UE_LOGF(LogPluginManager, Verbose, "Plugin path: %ls", *Plugin.FileName);

		UE::PluginManager::Private::CheckForDeprecatedPlugin(Plugin);

		// register this plugin, so it can be found right now, but also when loading other platforms, or when modifying it by other plugins
		const bool bIncludePluginNameInBranchName = false;
		DynamicLayerPriority LayerPrio = Plugin.GetType() == EPluginType::Engine ? DynamicLayerPriority::EnginePlugin : DynamicLayerPriority::ProjectPlugin;
		FConfigCacheIni::RegisterPlugin(PluginName, Plugin.GetBaseDir(), Plugin.GetExtensionBaseDirs(), LayerPrio, bIncludePluginNameInBranchName);

		FConfigContext ConfigContext = FConfigContext::ReadIntoGConfig();
		ConfigContext.ChangeTracker = &ChangeTracker;
		ConfigContext.ConfigFileTag = *Plugin.Name;
		ConfigContext.Load(*Plugin.Name);
	}

	// apply plugin modifications from this plugin to the everything
	FConfigCacheIni::AddMultiplePluginsToAllBranches(PluginNames);

	// walk over each plugin, modify existing ini's, mount the content, etc
	ParallelFor(PluginsArray.Num(), [&PluginsArray, &ConfigCS, &PluginPakCS, &ConfigFilesPluginsCannotOverride, &ChangeTracker, this](int32 Index)
		{
			FPlugin& Plugin = *PluginsArray[Index];
			FName PluginName(*Plugin.Name);

			// Build the list of content folders
			if (Plugin.Descriptor.bCanContainContent)
			{
				{
					FScopeLock Locker(&ConfigCS);

					// we need to look up the section each time because other loops could add entries
					if (FConfigFile* EngineConfigFile = GConfig->FindConfigFile(GEngineIni))
					{
						EngineConfigFile->AddUniqueToSection(TEXT("Core.System"), "Paths", Plugin.GetContentDir());
					}
				}

				TArray<FString>	FoundPaks;
				FPakFileSearchVisitor PakVisitor(FoundPaks);
				IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

				// Pak files are loaded from <PluginName>/Content/Paks/<PlatformName>
				if (FPlatformProperties::RequiresCookedData())
				{
					PlatformFile.IterateDirectoryRecursively(*(Plugin.GetContentDir() / TEXT("Paks") / FPlatformProperties::PlatformName()), PakVisitor);

					for (const FString& PakPath : FoundPaks)
					{
						FScopeLock Locker(&PluginPakCS);
						if (FCoreDelegates::MountPak.IsBound())
						{
							FCoreDelegates::MountPak.Execute(PakPath, 0);
							PluginsWithPakFile.AddUnique(PluginsArray[Index]);
						}
						else
						{
							UE_LOGF(LogPluginManager, Warning, "PAK file (%ls) could not be mounted because MountPak is not bound", *PakPath);
						}
					}
				}
			}
		}, true); // @todo disable parallelism for now as it's causing hard to track problems
}

void FPluginManager::MountContentPlugins(FPluginDiscoveryContext& Context)
{
	// Mount enabled plugins with content
	for (const FDiscoveredPluginMap::ElementType& PluginPair : AllPlugins)
	{
		const TSharedRef<FPlugin>& PluginRef = DiscoveredPluginMapUtils::ResolvePluginFromMapVal(PluginPair.Value);
		FPlugin& Plugin = *PluginRef;

		if (Plugin.IsEnabled())
		{
			if (Plugin.GetDescriptor().bExplicitlyLoaded)
			{
				continue;
			}

			Plugin.SetIsMounted(true);

			if ((Plugin.CanContainContent() || Plugin.CanContainVerse()))
			{
				if (NewPluginMountedEvent.IsBound())
				{
					NewPluginMountedEvent.Broadcast(Plugin);
				}

				if (ensure(RegisterMountPointDelegate.IsBound()))
				{
					FString ContentDir = Plugin.GetContentDir();
					RegisterMountPointDelegate.Execute(Plugin.GetMountedAssetPath(), ContentDir);
				}

				if (NewPluginContentMountedEvent.IsBound())
				{
					NewPluginContentMountedEvent.Broadcast(Plugin);
				}
			}
		}
	}
}

bool FPluginManager::ConfigureEnabledPlugins()
{
#if (WITH_ENGINE && !IS_PROGRAM) || WITH_PLUGIN_SUPPORT
	if(PluginsToConfigure.Num() > 0)
	{
		SCOPED_BOOT_TIMING("FPluginManager::ConfigureEnabledPlugins");

		bHaveAllRequiredPlugins = false;


		FPluginDiscoveryContext Context;

		// look for compiletime and commandline plugins
		FindCompileTimePlugins(Context);
		if (!FindCommandLinePlugins(Context))
		{
			return false;
		}

		// look for the engine enabled plugins
		if (!FindTargetPlugins(Context))
		{
			return false;
		}

		// look for program-specific plugins
		if (!FindProgramPlugins(Context))
		{
			return false;
		}

		MarkEnabledPlugins(Context);

		// If we made it here, we have all the required plugins
		bHaveAllRequiredPlugins = true;

		ProcessEnabledPlugins(Context);

		MountContentPlugins(Context);

		if (Context.AdditionalLocResPaths.Num() > 0)
		{
			FTextLocalizationManager::Get().HandleLocalizationTargetsMounted(Context.AdditionalLocResPaths);
		}

		PluginsToConfigure.Empty();
	}
	else
	{
		bHaveAllRequiredPlugins = true;
	}
	return bHaveAllRequiredPlugins;
#else
	return true;
#endif
}

bool FPluginManager::RequiresTempTargetForCodePlugin(const FProjectDescriptor* ProjectDescriptor, const FString& Platform, EBuildConfiguration Configuration, EBuildTargetType TargetType, FText& OutReason)
{
	FConfigurePluginResultInfo ResultInfo;

	TSet<FString> ProjectCodePlugins;
	if (!GetCodePluginsForProject(ProjectDescriptor, Platform, Configuration, TargetType, AllPlugins, ProjectCodePlugins, TSet<FString>(), ResultInfo))
	{
		OutReason = FText::Format(LOCTEXT("TempTarget_MissingPluginForTarget", "{0} plugin is referenced by target but not found"), FText::FromString(ResultInfo.PluginReference->Name));
		return true;
	}

	TSet<FString> DefaultCodePlugins;
	if (!GetCodePluginsForProject(nullptr, Platform, Configuration, TargetType, AllPlugins, DefaultCodePlugins, TSet<FString>(), ResultInfo))
	{
		OutReason = FText::Format(LOCTEXT("TempTarget_MissingPluginForDefaultTarget", "{0} plugin is referenced by the default target but not found"), FText::FromString(ResultInfo.PluginReference->Name));
		return true;
	}

	for (const FString& ProjectCodePlugin : ProjectCodePlugins)
	{
		if (!DefaultCodePlugins.Contains(ProjectCodePlugin))
		{
			OutReason = FText::Format(LOCTEXT("TempTarget_PluginEnabled", "{0} plugin is enabled"), FText::FromString(ProjectCodePlugin));
			return true;
		}
	}

	for (const FString& DefaultCodePlugin : DefaultCodePlugins)
	{
		if (!ProjectCodePlugins.Contains(DefaultCodePlugin))
		{
			OutReason = FText::Format(LOCTEXT("TempTarget_PluginDisabled", "{0} plugin is disabled"), FText::FromString(DefaultCodePlugin));
			return true;
		}
	}

	return false;
}

bool FPluginManager::GetCodePluginsForProject(const FProjectDescriptor* ProjectDescriptor, const FString& Platform,
	EBuildConfiguration Configuration, EBuildTargetType TargetType, FDiscoveredPluginMap& AllPlugins,
	TSet<FString>& CodePluginNames, const TSet<FString>& AllowedOptionalDependencies, FConfigurePluginResultInfo& OutResultInfo)
{
	// Can only check the current project at the moment, since we won't have enumerated them otherwise
	check(ProjectDescriptor == nullptr || ProjectDescriptor == IProjectManager::Get().GetCurrentProject());

	// Always false for content-only projects
	const bool bLoadPluginsForTargetPlatforms = (TargetType == EBuildTargetType::Editor);

	// Map of all enabled plugins
	TMap<FString, FPlugin*> EnabledPlugins;

	// Keep a set of all the plugin names that have been configured. We read configuration data from different places, but only configure a plugin from the first place that it's referenced.
	TSet<FString> ConfiguredPluginNames;

	// Find all the plugin references in the project file
	bool bAllowEnginePluginsEnabledByDefault = true;
	if (ProjectDescriptor != nullptr)
	{
		bAllowEnginePluginsEnabledByDefault = !ProjectDescriptor->bDisableEnginePluginsByDefault;

		// Copy the plugin references, since we may modify the project if any plugins are missing
		TArray<FPluginReferenceDescriptor> PluginReferences(ProjectDescriptor->Plugins);
		for (const FPluginReferenceDescriptor& PluginReference : PluginReferences)
		{
			if(!ConfiguredPluginNames.Contains(PluginReference.Name))
			{
				if (!ConfigureEnabledPluginForTarget(PluginReference, ProjectDescriptor, FString(), Platform,
					Configuration, TargetType, bLoadPluginsForTargetPlatforms, AllPlugins, EnabledPlugins,
					AllowedOptionalDependencies, OutResultInfo))
				{
					return false;
				}
				ConfiguredPluginNames.Add(PluginReference.Name);
			}
		}
	}

	// Add the plugins which are enabled by default
	for (const FDiscoveredPluginMap::ElementType& PluginPair : AllPlugins)
	{
		if (DiscoveredPluginMapUtils::ResolvePluginFromMapVal(PluginPair.Value)->IsEnabledByDefault(bAllowEnginePluginsEnabledByDefault) && !ConfiguredPluginNames.Contains(PluginPair.Key))
		{
			if (!ConfigureEnabledPluginForTarget(FPluginReferenceDescriptor(PluginPair.Key, true), ProjectDescriptor,
				FString(), Platform, Configuration, TargetType, bLoadPluginsForTargetPlatforms, AllPlugins,
				EnabledPlugins, AllowedOptionalDependencies, OutResultInfo))
			{
				return false;
			}
			ConfiguredPluginNames.Add(PluginPair.Key);
		}
	}

	// Figure out which plugins have code
	bool bBuildDeveloperTools = (TargetType == EBuildTargetType::Editor || TargetType == EBuildTargetType::Program || (Configuration != EBuildConfiguration::Test && Configuration != EBuildConfiguration::Shipping));
	bool bRequiresCookedData = (TargetType != EBuildTargetType::Editor);
	for (const TPair<FString, FPlugin*>& Pair : EnabledPlugins)
	{
		for (const FModuleDescriptor& Module : Pair.Value->GetDescriptor().Modules)
		{
			if (Module.IsCompiledInConfiguration(Platform, Configuration, FString(), TargetType, bBuildDeveloperTools, bRequiresCookedData))
			{
				CodePluginNames.Add(Pair.Key);
				break;
			}
		}
	}

	return true;
}

bool FPluginManager::ConfigureEnabledPluginForCurrentTarget(const FPluginReferenceDescriptor& FirstReference,
	TMap<FString, FPlugin*>& EnabledPlugins, FStringView SourceOfPluginRequest, const TSet<FString>& AllowedOptionalDependencies)
{
	FConfigurePluginResultInfo ResultInfo;

	if (ConfigureEnabledPluginForTarget(FirstReference, IProjectManager::Get().GetCurrentProject(), UE_APP_NAME,
		FPlatformMisc::GetUBTPlatform(), FApp::GetBuildConfiguration(), FApp::GetBuildTargetType(),
		(bool)LOAD_PLUGINS_FOR_TARGET_PLATFORMS, AllPlugins, EnabledPlugins, AllowedOptionalDependencies, ResultInfo))
	{
		return true;
	}

#if !IS_MONOLITHIC
	// If we're in unattended mode, don't open any windows
	if (!FApp::IsUnattended())
	{
		switch (ResultInfo.Code)
		{
		case EConfigurePluginResultCode::PluginMissing:
			{
				// Try to download it from the marketplace
				if (ResultInfo.PluginReference->MarketplaceURL.Len() > 0 && PromptToDownloadPlugin(ResultInfo.PluginReference->Name, ResultInfo.PluginReference->MarketplaceURL))
				{
					UE_LOGF(LogPluginManager, Display, "Downloading '%ls' plugin from marketplace (%ls).", *ResultInfo.PluginReference->Name, *ResultInfo.PluginReference->MarketplaceURL);
					return false;
				}

				// Prompt to disable it in the project file, if possible
				if (PromptToDisableMissingPlugin(FirstReference.Name, ResultInfo.PluginReference->Name))
				{
					UE_LOGF(LogPluginManager, Display, "Disabled plugin '%ls' due to missing plugin '%ls', continuing.", *FirstReference.Name, *ResultInfo.PluginReference->Name);
					return true;
				}
			}
			break;

		case EConfigurePluginResultCode::PluginSealed:
			{
				// Prompt to disable it in the project file, if possible
				if (PromptToDisableSealedPlugin(FirstReference.Name, ResultInfo.PluginReference->Name))
				{
					UE_LOGF(LogPluginManager, Display, "Disabled plugin '%ls' due to sealed dependency on plugin '%ls', continuing.", *FirstReference.Name, *ResultInfo.PluginReference->Name);
					return true;
				}
			}
			break;

		case EConfigurePluginResultCode::DisallowedDependency:
			{
				// Prompt to disable it in the project file, if possible
				if (PromptToDisableDisalowedPlugin(FirstReference.Name, ResultInfo.PluginReference->Name))
				{
					UE_LOGF(LogPluginManager, Display, "Disabled plugin '%ls' due to disallowed dependency on plugin '%ls', continuing.", *FirstReference.Name, *ResultInfo.PluginReference->Name);
					return true;
				}
			}
			break;

		default:
			checkf(0, TEXT("Unimplemented EConfigurePluginResultCode switch case"));
		}
	}
#endif

	// Unable to continue
	// If we're in unattended mode, set error to fatal, otherwise let the caller decide whether to kill the process.
	// Log more diagnostics in non-ship; don't give them in ship because they include full paths on the user's machine.
	bool bFatalError = FApp::IsUnattended();
	FString ErrorMessage;

#if UE_BUILD_SHIPPING
	ErrorMessage = FString::Printf(TEXT("Unable to load plugin '%s'. Aborting."), *ResultInfo.PluginReference->Name);
#else
	auto ReferenceChainToString = [](TArray<const FPluginReferenceDescriptor*>& Chain, const FPluginReferenceDescriptor* End)
		{
			TStringBuilder<256> ChainStr;
			for (const FPluginReferenceDescriptor* Iter : Chain)
			{
				ChainStr << Iter->Name << TEXTVIEW(" -> ");
			}
			ChainStr << End->Name;
			return FString(ChainStr);
		};
	TArray<FString> PluginLocations;
	FDiscoveredPluginMap UnusedListOfDiscoveredPlugins;
	ReadAllPlugins(UnusedListOfDiscoveredPlugins, PluginDiscoveryPaths, &PluginLocations);
	TStringBuilder<1024> PluginLocationsStr;
	for (const FString& Location : PluginLocations)
	{
		PluginLocationsStr << TEXT("\n\t") << Location;
	}
	if (ResultInfo.PluginReference == &FirstReference)
	{
		switch (ResultInfo.Code)
		{
		case EConfigurePluginResultCode::PluginMissing:
			ErrorMessage = FString::Printf(
				TEXT("Unable to load plugin '%s'. It was requested by %.*s, but is missing on disk. Aborting. Looked in these locations for .uplugin files:")
				TEXT("%s"),
				*FirstReference.Name, SourceOfPluginRequest.Len(), SourceOfPluginRequest.GetData(),
				*PluginLocationsStr);
			break;
		case EConfigurePluginResultCode::PluginSealed:
			ErrorMessage = FString::Printf(
				TEXT("Unable to load plugin '%s'. It was requested by %.*s, but is is a sealed plugin and is therefore not usable. Aborting."),
				*FirstReference.Name, SourceOfPluginRequest.Len(), SourceOfPluginRequest.GetData());
			break;
		case EConfigurePluginResultCode::DisallowedDependency:
			ErrorMessage = FString::Printf(
				TEXT("Unable to load plugin '%s'. It was requested by %.*s, but is a disallowed plugin is therefore not usable. Aborting."),
				*FirstReference.Name, SourceOfPluginRequest.Len(), SourceOfPluginRequest.GetData());
			break;
		default:
			checkf(0, TEXT("Unimplemented EConfigurePluginResultCode switch case"));
		}
	}
	else
	{
		switch (ResultInfo.Code)
		{
		case EConfigurePluginResultCode::PluginMissing:
			ErrorMessage = FString::Printf(
				TEXT("Unable to load plugin '%s'. It was requested by %.*s, but it has a dependency that is missing on disk.")
				TEXT("\n\tThe missing dependency is %s (ReferenceChain: %s).")
				TEXT("\n\tAborting.")
				TEXT("\n\tLooked in these locations for .uplugin files:")
				TEXT("%s"),
				*FirstReference.Name, SourceOfPluginRequest.Len(), SourceOfPluginRequest.GetData(),
				*ResultInfo.PluginReference->Name, *ReferenceChainToString(ResultInfo.ReferenceChain, ResultInfo.PluginReference),
				*PluginLocationsStr);
			break;
		case EConfigurePluginResultCode::PluginSealed:
			ErrorMessage = FString::Printf(
				TEXT("Unable to load plugin '%s'. It was requested by %.*s, but it has a dependency on a sealed plugin.")
				TEXT("\n\tThe sealed dependency is %s (ReferenceChain: %s).")
				TEXT("\n\tAborting."),
				*FirstReference.Name, SourceOfPluginRequest.Len(), SourceOfPluginRequest.GetData(),
				*ResultInfo.PluginReference->Name, *ReferenceChainToString(ResultInfo.ReferenceChain, ResultInfo.PluginReference));
			break;
		case EConfigurePluginResultCode::DisallowedDependency:
			ErrorMessage = FString::Printf(
				TEXT("Unable to load plugin '%s'. It was requested by %.*s, but it has a dependency on a disallowed plugin.")
				TEXT("\n\tThe disallowed plugin is %s (ReferenceChain: %s).")
				TEXT("\n\tAborting."),
				*FirstReference.Name, SourceOfPluginRequest.Len(), SourceOfPluginRequest.GetData(),
				*ResultInfo.PluginReference->Name, *ReferenceChainToString(ResultInfo.ReferenceChain, ResultInfo.PluginReference));
			break;
		default:
			checkf(0, TEXT("Unimplemented EConfigurePluginResultCode switch case"));
		}
	}
#endif

	if (bFatalError)
	{
		UE_LOGF(LogPluginManager, Fatal, "%ls", *ErrorMessage);
	}
	else
	{
		UE_LOGF(LogPluginManager, Error, "%ls", *ErrorMessage);
	}
	return false;
}

bool FPluginManager::ConfigureEnabledPluginForTarget(const FPluginReferenceDescriptor& FirstReference,
	const FProjectDescriptor* ProjectDescriptor, const FString& TargetName, const FString& Platform,
	EBuildConfiguration Configuration, EBuildTargetType TargetType, bool bLoadPluginsForTargetPlatforms,
	FDiscoveredPluginMap& AllPlugins, TMap<FString, FPlugin*>& EnabledPlugins,
	const TSet<FString>& AllowedOptionalDependencies,
	FConfigurePluginResultInfo& OutResultInfo)
{
	if (EnabledPlugins.Contains(FirstReference.Name))
	{
		// Already enabled. Just verify the version
		if (FirstReference.RequestedVersion.IsSet())
		{
			UE_CLOGF(
				(*EnabledPlugins.Find(FirstReference.Name))->GetDescriptor().Version != FirstReference.RequestedVersion.GetValue(),
				LogPluginManager, Error,
				"Requested explicit version (v%d) of plugin '%ls', but a different version (v%d) was already enabled by another source.",
				FirstReference.RequestedVersion.GetValue(), *FirstReference.Name,
				(*EnabledPlugins.Find(FirstReference.Name))->GetDescriptor().Version
			);
		}
		return true;
	}

	if (FirstReference.TargetsToExplicitlyLoad.Contains(TargetType))
	{
		UE_LOGF(LogPluginManager, Verbose, "Ignoring plugin '%ls' for being set to load-on-demand in project.", *FirstReference.Name);
		return true;
	}

	// Set of plugin names we've added to the queue for processing, and the plugin that referenced
	// them, so we can display the chain of references if necessary for diagnostics
	TMap<FString, const FPluginReferenceDescriptor*> SeenPlugins;

	// Queue of plugin references to consider
	TArray<const FPluginReferenceDescriptor*> NewPluginQueue;

	OutResultInfo.ReferenceChain.Reset();
	auto SetReferenceChain = [&SeenPlugins, &OutResultInfo](const FPluginReferenceDescriptor* Reference)
		{
			while (Reference)
			{
				const FPluginReferenceDescriptor** Next = SeenPlugins.Find(Reference->Name);
				// SeenPlugins[FIrstReference] == FirstReference, so stop when we reach that cycle. No other cycles are possible.
				if (!Next || Reference == *Next)
				{
					break;
				}
				Reference = *Next;
				// The reference chain should not include the input Reference, so add at the end of loop rather than the beginning
				OutResultInfo.ReferenceChain.Add(Reference);
			}
			Algo::Reverse(OutResultInfo.ReferenceChain);
		};


	// Loop through the queue of plugin references that need to be enabled, queuing more items as we go
	NewPluginQueue.Add(&FirstReference);
	SeenPlugins.Add(FirstReference.Name, &FirstReference);
	for (int32 Idx = 0; Idx < NewPluginQueue.Num(); Idx++)
	{
		const FPluginReferenceDescriptor& Reference = *NewPluginQueue[Idx];

		// Check if the plugin is required for this platform
		if(!Reference.IsEnabledForPlatform(Platform) || !Reference.IsEnabledForTargetConfiguration(Configuration) || !Reference.IsEnabledForTarget(TargetType))
		{
			UE_LOGF(LogPluginManager, Verbose, "Ignoring plugin '%ls' for platform/configuration", *Reference.Name);
			continue;
		}

		// Check if the plugin is required for this platform
		if(!bLoadPluginsForTargetPlatforms && !Reference.IsSupportedTargetPlatform(Platform))
		{
			UE_LOGF(LogPluginManager, Verbose, "Ignoring plugin '%ls' due to unsupported platform", *Reference.Name);
			continue;
		}

		// Find the plugin being enabled
		const TSharedRef<FPlugin>* PluginPtr = DiscoveredPluginMapUtils::FindPluginInMap(AllPlugins, Reference.Name);
		if (PluginPtr && Reference.RequestedVersion.IsSet())
		{
			TArray<TSharedRef<FPlugin>>* PluginVersions = DiscoveredPluginMapUtils::FindAllPluginVersionsWithName(AllPlugins, Reference.Name);
			check(PluginVersions);

			TSharedRef<FPlugin>* FoundVersion = nullptr;
			for (TSharedRef<FPlugin>& PluginVersion : *PluginVersions)
			{
				if (PluginVersion->GetDescriptor().Version == Reference.RequestedVersion.GetValue())
				{
					FoundVersion = &PluginVersion;
					break;
				}
			}

			if (!FoundVersion)
			{
				UE_LOGF(LogPluginManager, Warning, "Failed to find specific version (v%d) of plugin '%ls'. Other versions exist, but the explicit version requested was missing.",
					Reference.RequestedVersion.GetValue(), *Reference.Name);

				PluginPtr = nullptr;
			}
			else if (FoundVersion != PluginPtr && (*PluginPtr)->bEnabled)
			{
				UE_LOGF(LogPluginManager, Warning, "A different version of the %ls plugin (v%d : '%ls') is already enabled. Cannot switch to v%d (%ls) while another version is active.",
					*Reference.Name, (*PluginPtr)->Descriptor.Version, *(*PluginPtr)->FileName, Reference.RequestedVersion.GetValue(), *(*FoundVersion)->FileName);

				PluginPtr = nullptr;
			}
			else
			{
				PluginPtr = DiscoveredPluginMapUtils::PromotePluginToOfferedVersion(AllPlugins, FoundVersion);
			}
		}

		if (PluginPtr == nullptr)
		{
			// Ignore any optional plugins
			if (Reference.bOptional)
			{
				UE_LOGF(LogPluginManager, Verbose, "Ignored optional reference to '%ls' plugin; plugin was not found.", *Reference.Name);
				continue;
			}

			// Add it to the missing list
			OutResultInfo = FConfigurePluginResultInfo(EConfigurePluginResultCode::PluginMissing, &Reference);
			SetReferenceChain(&Reference);
			return false;
		}

		// Check the plugin is not disabled by the platform
		FPlugin& Plugin = PluginPtr->Get();

		// Allow the platform to disable it
		if (FPlatformMisc::ShouldDisablePluginAtRuntime(Plugin.Name))
		{
			UE_LOGF(LogPluginManager, Verbose, "Plugin '%ls' was disabled by platform.", *Reference.Name);
			continue;
		}

		// Check the plugin supports this platform
		if (!bLoadPluginsForTargetPlatforms && !Plugin.Descriptor.SupportsTargetPlatform(Platform))
		{
			UE_LOGF(LogPluginManager, Verbose, "Ignoring plugin '%ls' due to unsupported platform in plugin descriptor", *Reference.Name);
			continue;
		}

		// Check that this plugin supports the current program
		if (TargetType == EBuildTargetType::Program && !Plugin.Descriptor.SupportedPrograms.Contains(TargetName))
		{
			UE_LOGF(LogPluginManager, Verbose, "Ignoring plugin '%ls' due to absence from the supported programs list", *Reference.Name);
			continue;
		}

		// Skip loading Enterprise plugins when project is not an Enterprise project
		if (Plugin.Type == EPluginType::Enterprise && ProjectDescriptor != nullptr && !ProjectDescriptor->bIsEnterpriseProject)
		{
			UE_LOGF(LogPluginManager, Verbose, "Ignoring plugin '%ls' due to not being an Enterpise project", *Reference.Name);
			continue;
		}

#if IS_MONOLITHIC || UE_MERGED_MODULES
		// In monolithic or merged-modular builds, if the plugin is optional, and any modules are not compiled in this build, do not load it. This makes content compiled with plugins enabled compatible with builds where they are disabled.
		if (Reference.bOptional)
		{
			bool bAllModulesAvailable = true;
			for (const FModuleDescriptor& ModuleDescriptor : Plugin.Descriptor.Modules)
			{
				if (ModuleDescriptor.IsLoadedInCurrentConfiguration() && !FModuleManager::Get().ModuleExists(*ModuleDescriptor.Name.ToString()))
				{
					bAllModulesAvailable = false;
					break;
				}
			}

			if (!bAllModulesAvailable)
			{
				UE_LOGF(LogPluginManager, Verbose, "Ignored optional reference to '%ls' plugin; plugin's modules were not found.", *Reference.Name);
				continue;
			}
		}
#endif // IS_MONOLITHIC || UE_MERGED_MODULES

#if !UE_BUILD_SHIPPING
		if (Reference.bOptional && !AllowedOptionalDependencies.IsEmpty() && !AllowedOptionalDependencies.Contains(Reference.Name))
		{
			UE_LOGF(LogPluginManager, Verbose, "Ignored optional reference to '%ls' plugin; plugin was not built by target.", *Reference.Name);
			continue;
		}
#endif

		// Add references to all its dependencies
		for (const FPluginReferenceDescriptor& NextReference : Plugin.Descriptor.Plugins)
		{
			const TSharedRef<FPlugin>* NextPluginPtr = DiscoveredPluginMapUtils::FindPluginInMap(AllPlugins, NextReference.Name);
			bool bIsNewlySeen = false;
			const FPluginReferenceDescriptor*& SeenReferencer = SeenPlugins.FindOrAdd(NextReference.Name, nullptr);
			if (!SeenReferencer)
			{
				bIsNewlySeen = true;
				SeenReferencer = &Reference;
			}

			if (NextPluginPtr != nullptr && NextPluginPtr->Get().Descriptor.bIsSealed)
			{
				OutResultInfo = FConfigurePluginResultInfo(EConfigurePluginResultCode::PluginSealed, &NextReference);
				SetReferenceChain(&NextReference);
				return false;
			}

			if (NextPluginPtr != nullptr && Plugin.Descriptor.DisallowedPlugins.ContainsByPredicate(
				[&NextPluginPtr](const FPluginDisallowedDescriptor& Other) { return Other.Name == (*NextPluginPtr)->Name; }))
			{
				OutResultInfo = FConfigurePluginResultInfo(EConfigurePluginResultCode::DisallowedDependency, &NextReference);
				// If the dependency was previously referenced by another plugin, overwrite that other plugin with the current Plugin,
				// so that the caller will see the reference chain that has the DisallowedDependencye
				SeenReferencer = &Reference;
				SetReferenceChain(&NextReference);
				return false;
			}

#if !UE_BUILD_SHIPPING
			// Allowed optional plugins are compiled enabled or enabled via the commandline. Ignore those that are not.
			if (NextReference.bOptional && !AllowedOptionalDependencies.IsEmpty() && !AllowedOptionalDependencies.Contains(NextReference.Name))
			{
				UE_LOGF(LogPluginManager, Verbose, "Ignored optional reference to '%ls' plugin from '%ls' plugin; plugin was not built by target.", *NextReference.Name, *Plugin.GetName());
				continue;
			}
#endif

			if (!EnabledPlugins.Contains(NextReference.Name) && bIsNewlySeen)
			{
				NewPluginQueue.Add(&NextReference);
			}
		}

		// Add the plugin
		EnabledPlugins.Add(Plugin.GetName(), &Plugin);
	}
	return true;
}

bool FPluginManager::PromptToDownloadPlugin(const FString& PluginName, const FString& MarketplaceURL)
{
	if(MarketplaceURL.StartsWith("https://"))
	{
		FText Caption = FText::Format(LOCTEXT("DownloadPluginCaption", "Missing {0} Plugin"), FText::FromString(PluginName));
		FText Message = FText::Format(LOCTEXT("DownloadPluginMessage", "This project requires the {0} plugin.\n\nWould you like to download it from the Unreal Engine Marketplace?"), FText::FromString(PluginName));
		if(FMessageDialog::Open(EAppMsgType::YesNo, Message, Caption) == EAppReturnType::Yes)
		{
			FString Error;
			FPlatformProcess::LaunchURL(*MarketplaceURL, nullptr, &Error);
			if (Error.Len() == 0)
			{
				return true;
			}
			FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(Error));
		}
	}
	return false;
}

bool FPluginManager::PromptToDisableSealedPlugin(const FString& PluginName, const FString& SealedPluginName)
{
	FText Message = FText::Format(LOCTEXT("DisablePluginMessage_IllegalDependency", "This project requires the '{0}' plugin, which has an illegal dependency on the sealed '{1}' plugin.\n\nWould you like to disable it?\n\nIf you do, you will no longer be able to open any assets created with it. If not, the application will close."), FText::FromString(PluginName), FText::FromString(SealedPluginName));
	FText Caption(LOCTEXT("DisablePluginCaption_IllegalDependency", "Illegal Dependency"));

	return PromptToDisablePlugin(Caption, Message, PluginName);
}

bool FPluginManager::PromptToDisableDisalowedPlugin(const FString& PluginName, const FString& DisallowedPluginName)
{
	FText Message = FText::Format(LOCTEXT("DisablePluginMessage_DisallowedDependency", "This project requires the '{0}' plugin, which has a disallowed dependency on the '{1}' plugin.\n\nWould you like to disable it?\n\nIf you do, you will no longer be able to open any assets created with it. If not, the application will close."), FText::FromString(PluginName), FText::FromString(DisallowedPluginName));
	FText Caption(LOCTEXT("DisablePluginCaption_IllegalDependency", "Illegal Dependency"));

	return PromptToDisablePlugin(Caption, Message, PluginName);
}

bool FPluginManager::PromptToDisableMissingPlugin(const FString& PluginName, const FString& MissingPluginName)
{
	FText Message;
	if (PluginName == MissingPluginName)
	{
		Message = FText::Format(LOCTEXT("DisablePluginMessage_NotFound", "This project requires the '{0}' plugin, which could not be found. Would you like to disable it and continue?\n\nIf you do, you will no longer be able to open any assets created with it. If not, the application will close."), FText::FromString(PluginName));
	}
	else
	{
		Message = FText::Format(LOCTEXT("DisablePluginMessage_MissingDependency", "This project requires the '{0}' plugin, which has a missing dependency on the '{1}' plugin.\n\nWould you like to disable it?\n\nIf you do, you will no longer be able to open any assets created with it. If not, the application will close."), FText::FromString(PluginName), FText::FromString(MissingPluginName));
	}

	FText Caption(LOCTEXT("DisablePluginCaption", "Missing Plugin"));
	return PromptToDisablePlugin(Caption, Message, PluginName);
}

bool FPluginManager::PromptToDisableIncompatiblePlugin(const FString& PluginName, const FString& IncompatiblePluginName)
{
	FText Message;
	if (PluginName == IncompatiblePluginName)
	{
		Message = FText::Format(LOCTEXT("DisablePluginMessage_MissingOrIncompatibleEngineVersion", "Binaries for the '{0}' plugin are missing or incompatible with the current engine version.\n\nWould you like to disable it? You will no longer be able to open assets that were created with it."), FText::FromString(PluginName));
	}
	else
	{
		Message = FText::Format(LOCTEXT("DisablePluginMessage_MissingOrIncompatibleDependency", "Binaries for the '{0}' plugin (a dependency of '{1}') are missing or incompatible with the current engine version.\n\nWould you like to disable it? You will no longer be able to open assets that were created with it."), FText::FromString(IncompatiblePluginName), FText::FromString(PluginName));
	}

	FText Caption(LOCTEXT("DisablePluginCaption", "Missing Plugin"));
	return PromptToDisablePlugin(Caption, Message, PluginName);
}

bool FPluginManager::PromptToDisablePlugin(const FText& Caption, const FText& Message, const FString& PluginName)
{
	// Check we have a project file. If this is a missing engine/program plugin referenced by something, we can't disable it through this method.
	if (IProjectManager::Get().GetCurrentProject() != nullptr)
	{
		if (FMessageDialog::Open(EAppMsgType::YesNo, Message, Caption) == EAppReturnType::Yes)
		{
			FText FailReason;
			if (IProjectManager::Get().SetPluginEnabled(*PluginName, false, FailReason))
			{
				return true;
			}
			FMessageDialog::Open(EAppMsgType::Ok, FailReason);
		}
	}
	return false;
}

bool FPluginManager::IsPluginCompatible(const FPlugin& Plugin)
{
	if (Plugin.Descriptor.EngineVersion.Len() > 0)
	{
		FEngineVersion Version;
		if (!FEngineVersion::Parse(Plugin.Descriptor.EngineVersion, Version))
		{
			UE_LOGF(LogPluginManager, Warning, "Engine version string in %ls could not be parsed (\"%ls\")", *Plugin.FileName, *Plugin.Descriptor.EngineVersion);
			return true;
		}

		EVersionComparison Comparison = FEngineVersion::GetNewest(FEngineVersion::CompatibleWith(), Version, nullptr);
		if (Comparison != EVersionComparison::Neither)
		{
			UE_LOGF(LogPluginManager, Warning, "Plugin '%ls' requires engine version '%ls' and may not be compatible with the current current engine version '%ls'", *Plugin.Name, *Plugin.Descriptor.EngineVersion, *FEngineVersion::CompatibleWith().ToString());
			return false;
		}
	}
	return true;
}

bool FPluginManager::PromptToLoadIncompatiblePlugin(const FPlugin& Plugin)
{
	// Format the message dependning on whether the plugin is referenced directly, or as a dependency
	FText Message = FText::Format(LOCTEXT("LoadIncompatiblePlugin", "The '{0}' plugin was designed for build {1}. Attempt to load it anyway?"), FText::FromString(Plugin.Name), FText::FromString(Plugin.Descriptor.EngineVersion));
	FText Caption = FText::Format(LOCTEXT("IncompatiblePluginCaption", "'{0}' is Incompatible"), FText::FromString(Plugin.Name));
	return FMessageDialog::Open(EAppMsgType::YesNo, Message, Caption) == EAppReturnType::Yes;
}

TSharedPtr<FPlugin> FPluginManager::FindPluginInstance(const FString& Name)
{
	const TSharedRef<FPlugin>* Instance = DiscoveredPluginMapUtils::FindPluginInMap(AllPlugins, Name);
	if (Instance == nullptr)
	{
		return TSharedPtr<FPlugin>();
	}
	else
	{
		return TSharedPtr<FPlugin>(*Instance);
	}
}


bool FPluginManager::TryLoadModulesForPlugin( const FPlugin& Plugin, const ELoadingPhase::Type LoadingPhase ) const
{
	TMap<FName, EModuleLoadResult> ModuleLoadFailures;
	FModuleDescriptor::LoadModulesForPhase(LoadingPhase, Plugin.Descriptor.Modules, ModuleLoadFailures);

	FText FailureMessage;
	for( auto FailureIt( ModuleLoadFailures.CreateConstIterator() ); FailureIt; ++FailureIt )
	{
		const FName ModuleNameThatFailedToLoad = FailureIt.Key();
		const EModuleLoadResult FailureReason = FailureIt.Value();

		if( FailureReason != EModuleLoadResult::Success )
		{
			const FText PluginNameText = FText::FromString(Plugin.Name);
			const FText TextModuleName = FText::FromName(FailureIt.Key());

			if ( FailureReason == EModuleLoadResult::FileNotFound )
			{
				FailureMessage = FText::Format( LOCTEXT("PluginModuleNotFound", "Plugin '{0}' failed to load because module '{1}' could not be found.  Please ensure the plugin is properly installed, otherwise consider disabling the plugin for this project."), PluginNameText, TextModuleName );
			}
			else if ( FailureReason == EModuleLoadResult::FileIncompatible )
			{
				FailureMessage = FText::Format( LOCTEXT("PluginModuleIncompatible", "Plugin '{0}' failed to load because module '{1}' does not appear to be compatible with the current version of the engine.  The plugin may need to be recompiled."), PluginNameText, TextModuleName );
			}
			else if ( FailureReason == EModuleLoadResult::CouldNotBeLoadedByOS )
			{
				FailureMessage = FText::Format( LOCTEXT("PluginModuleCouldntBeLoaded", "Plugin '{0}' failed to load because module '{1}' could not be loaded.  There may be an operating system error or the module may not be properly set up."), PluginNameText, TextModuleName );
			}
			else if ( FailureReason == EModuleLoadResult::FailedToInitialize )
			{
				FailureMessage = FText::Format( LOCTEXT("PluginModuleFailedToInitialize", "Plugin '{0}' failed to load because module '{1}' could not be initialized successfully after it was loaded."), PluginNameText, TextModuleName );
			}
			else
			{
				ensure(0);	// If this goes off, the error handling code should be updated for the new enum values!
				FailureMessage = FText::Format( LOCTEXT("PluginGenericLoadFailure", "Plugin '{0}' failed to load because module '{1}' could not be loaded for an unspecified reason.  This plugin's functionality will not be available.  Please report this error."), PluginNameText, TextModuleName );
			}

			// Don't need to display more than one module load error per plugin that failed to load
			break;
		}
	}

	if( !FailureMessage.IsEmpty() )
	{
		if (bAllPluginsEnabledViaCommandLine)
		{
			UE_LOGF(LogPluginManager, Display, "%ls", *FailureMessage.ToString());
		}
		else
		{
			UE_LOGF(LogPluginManager, Error, "%ls", *FailureMessage.ToString());
		}
		if (!bAllPluginsEnabledViaCommandLine)
		{
			FMessageDialog::Open(EAppMsgType::Ok, FailureMessage);
			return false;
		}
	}

	return true;
}

bool FPluginManager::TryUnloadModulesForPlugin(const FPlugin& Plugin, const ELoadingPhase::Type LoadingPhase, FText* OutFailureMessage /*= nullptr*/, bool bSkipUnload /*= false*/, bool bAllowUnloadCode /*= true*/) const
{
	TMap<FName, EModuleUnloadResult> Errors;
	FModuleDescriptor::UnloadModulesForPhase(LoadingPhase, Plugin.Descriptor.Modules, Errors, bSkipUnload, bAllowUnloadCode);

	FText FailureMessage;
	for( const TPair<FName, EModuleUnloadResult>& FailureIt : Errors)
	{
		const FName ModuleNameThatFailedToLoad = FailureIt.Key;
		const EModuleUnloadResult FailureReason = FailureIt.Value;

		if (FailureReason != EModuleUnloadResult::Success)
		{
			const FText PluginNameText = FText::FromString(Plugin.Name);
			const FText TextModuleName = FText::FromName(ModuleNameThatFailedToLoad);

			if (FailureReason == EModuleUnloadResult::UnloadNotSupported)
			{
				FailureMessage = FText::Format(LOCTEXT("UnloadNotSupported", "Plugin '{0}' failed to unload because module '{1}' does not support unloading."), PluginNameText, TextModuleName);
			}
			else
			{
				ensure(0);	// If this goes off, the error handling code should be updated for the new enum values!
				FailureMessage = FText::Format(LOCTEXT("PluginGenericUnloadFailure", "Plugin '{0}' failed to unload because module '{1}' could not be unloaded for an unspecified reason. Please report this error."), PluginNameText, TextModuleName);
			}

			// Don't need to display more than one module load error per plugin that failed to load
			break;
		}
	}

	if (!FailureMessage.IsEmpty())
	{
		UE_LOGF(LogPluginManager, Error, "%ls", *FailureMessage.ToString());

		FMessageDialog::Open(EAppMsgType::Ok, FailureMessage);

		if (OutFailureMessage)
		{
			*OutFailureMessage = MoveTemp(FailureMessage);
		}

		return false;
	}

	return true;
}

bool FPluginManager::LoadModulesForEnabledPlugins( const ELoadingPhase::Type LoadingPhase )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPluginManager::LoadModulesForEnabledPlugins);

	// Figure out which plugins are enabled
	bool bSuccess = true;
	if (!ConfigureEnabledPlugins())
	{
		bSuccess = false;
	}
	else
	{
		const FStringView LoadingPhaseAsString = FStringView(ELoadingPhase::ToString(LoadingPhase));
		FScopedSlowTask SlowTask((float)AllPlugins.Num(),
			FText::Format(LOCTEXT("LoadingModulesForEnabledPlugins", "Loading Plugin Modules for Phase {0}"), FText::FromStringView(LoadingPhaseAsString)));
		SlowTask.Visibility = ESlowTaskVisibility::Important; // this function can be very slow, users will benefit from our messages
		LLM_SCOPE_BYNAME(TEXT("Modules"));

		// Load plugins!
		int32 NumProcessedSinceProgress = 0;
		for (const FDiscoveredPluginMap::ElementType& PluginPair : AllPlugins)
		{
			++NumProcessedSinceProgress;
			const TSharedRef<FPlugin>& Plugin = DiscoveredPluginMapUtils::ResolvePluginFromMapVal(PluginPair.Value);

			if (Plugin->bEnabled && !Plugin->Descriptor.bExplicitlyLoaded)
			{
				const FText Message = FText::Format(LOCTEXT("LoadingModulesForPlugin", "Loading {0} Modules for Plugin: {1}"),
					FText::FromStringView(LoadingPhaseAsString),
					FText::FromString(Plugin->Name));
				SlowTask.EnterProgressFrame((float)NumProcessedSinceProgress, Message);
				NumProcessedSinceProgress = 0;

				if (!TryLoadModulesForPlugin(Plugin.Get(), LoadingPhase))
				{
					bSuccess = false;
					break;
				}
			}
		}

#if !UE_MERGED_MODULES

		// This is a workaround for crashes when delaying the loading of binaries when using pak files/iostore. Should be removed.
		// This is avoided while using merged modular build, as we don't want modules to be loaded except when we're certain to need them.
		if (bPreloadedBinaries && LoadingPhase == ELoadingPhase::PostConfigInit)
		{
			UE_SCOPED_ENGINE_ACTIVITY("Preloading all plugin binaries");
			for (const FDiscoveredPluginMap::ElementType& PluginPair : AllPlugins)
			{
				if (!CanEnablePluginInCurrentTarget(PluginPair.Key))
				{
					UE_LOGF(LogPluginManager, Display, "Skipping loading modules for %ls. Plugin is not enabled for current target.", *PluginPair.Key);
					continue;
				}
				for (auto& ModuleName : DiscoveredPluginMapUtils::ResolvePluginFromMapVal(PluginPair.Value)->Descriptor.Modules)
				{
					if (ModuleName.IsCompiledInCurrentConfiguration())
					{
						FModuleManager::Get().LoadModuleBinaryOnly(ModuleName.Name);
					}
				}
			}
		}

#endif // !UE_MERGED_MODULES
	}
	// Some phases such as ELoadingPhase::PreEarlyLoadingScreen are potentially called multiple times,
	// but we do not return to an earlier phase after calling LoadModulesForEnabledPlugins on a later phase
	UE_CLOGF(LastCompletedLoadingPhase != ELoadingPhase::None && LastCompletedLoadingPhase > LoadingPhase,
		LogPluginManager, Error, "LoadModulesForEnabledPlugins called on phase %d after already being called on later phase %d.",
		static_cast<int32>(LoadingPhase), static_cast<int32>(LastCompletedLoadingPhase));

#if WITH_ENGINE
	// Engine Startup needs to process all phases, even phases not relevant to the current engine configuration,
	// (e.g. PostSplashScreen on Server) because some plugins might load themselves on that phase to ensure
	// they occur before/after related code is intialized on the platforms that have code related to the phase
	// but they still need to be loaded at SOME point on the platforms that do not have any related code. Rather
	// than writing code to find those plugins and have them register on the next phase that occurs, just make
	// sure that every platform processes every phase.
	// Since the constraint enforced above means we can never go backwards in phases, this constraint means that
	// every new phase must be the exact successor of the last completed phase (or a repeat of the last phase).
	UE_CLOGF((LastCompletedLoadingPhase == ELoadingPhase::None && LoadingPhase != ELoadingPhase::EarliestPossible)
		|| (LastCompletedLoadingPhase != ELoadingPhase::None
			&& static_cast<int32>(LoadingPhase) > static_cast<int32>(LastCompletedLoadingPhase) + 1),
		LogPluginManager, Error,
		"LoadModulesForEnabledPlugins called on phase %d after last being called on phase %d, which skipped phase %d.",
		static_cast<int32>(LoadingPhase), static_cast<int32>(LastCompletedLoadingPhase),
		static_cast<int32>(LastCompletedLoadingPhase) + 1);
#endif

	// We send the broadcast event each time, even if this function is called multiple times with the same phase
	LastCompletedLoadingPhase = LoadingPhase;
	LoadingPhaseCompleteEvent.Broadcast(LoadingPhase, bSuccess);
	return bSuccess;
}

IPluginManager::FLoadingModulesForPhaseEvent& FPluginManager::OnLoadingPhaseComplete()
{
	return LoadingPhaseCompleteEvent;
}

ELoadingPhase::Type FPluginManager::GetLastCompletedLoadingPhase() const
{
	return LastCompletedLoadingPhase;
}

void FPluginManager::GetLocalizationPathsForEnabledPlugins( TArray<FString>& OutLocResPaths )
{
	// Note: We don't call ConfigureEnabledPlugins here as it can cause additional plugin modules to load from a worker thread
	//       We expect that newly enabled plugins call HandleLocalizationTargetsMounted to load their localization target data

	// Gather the paths from all plugins that have localization targets that are loaded based on the current runtime environment
	for (const FDiscoveredPluginMap::ElementType& PluginPair : AllPlugins)
	{
		const TSharedRef<FPlugin>& Plugin = DiscoveredPluginMapUtils::ResolvePluginFromMapVal(PluginPair.Value);
		if (!Plugin->bEnabled || (Plugin->GetDescriptor().bExplicitlyLoaded && Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount == 0) || Plugin->GetDescriptor().LocalizationTargets.Num() == 0)
		{
			continue;
		}

		PluginLocalizationUtils::GetLocalizationPathsForPlugin(*Plugin, OutLocResPaths);
	}
}

void FPluginManager::SetRegisterMountPointDelegate( const FRegisterMountPointDelegate& Delegate )
{
	RegisterMountPointDelegate = Delegate;
}

void FPluginManager::SetUnRegisterMountPointDelegate( const FRegisterMountPointDelegate& Delegate )
{
	UnRegisterMountPointDelegate = Delegate;
}

bool FPluginManager::AreRequiredPluginsAvailable()
{
	return ConfigureEnabledPlugins();
}

#if !IS_MONOLITHIC
bool FPluginManager::CheckModuleCompatibility(TArray<FString>& OutIncompatibleModules, TArray<FString>& OutIncompatibleEngineModules)
{
	if(!ConfigureEnabledPlugins())
	{
		return false;
	}

	bool bResult = true;
	for(const FDiscoveredPluginMap::ElementType& PluginPair : AllPlugins)
	{
		const TSharedRef< FPlugin > &Plugin = DiscoveredPluginMapUtils::ResolvePluginFromMapVal(PluginPair.Value);

		TArray<FString> IncompatibleModules;
		if (Plugin->bEnabled && !FModuleDescriptor::CheckModuleCompatibility(Plugin->Descriptor.Modules, IncompatibleModules))
		{
			OutIncompatibleModules.Append(IncompatibleModules);
			if (Plugin->GetLoadedFrom() == EPluginLoadedFrom::Engine)
			{
				OutIncompatibleEngineModules.Append(IncompatibleModules);
			}
			bResult = false;
		}
	}
	return bResult;
}
#endif

IPluginManager& IPluginManager::Get()
{
	// Single instance of manager, allocated on demand and destroyed on program exit.
	static FPluginManager PluginManager;
	return PluginManager;
}

TSharedPtr<IPlugin> FPluginManager::FindPlugin(const FStringView Name)
{
	const uint32 NameHash = GetTypeHash(Name);
	const TSharedRef<FPlugin>* Instance = DiscoveredPluginMapUtils::FindPluginInMap_ByHash(AllPlugins, NameHash, Name);
	if (Instance == nullptr)
	{
		return TSharedPtr<IPlugin>();
	}
	else
	{
		return TSharedPtr<IPlugin>(*Instance);
	}
}

TSharedPtr<IPlugin> FPluginManager::FindPluginFromPath(const FString& PluginPath)
{
	FString PluginName = PluginPath;
	FPaths::NormalizeFilename(PluginName);

	if (PluginName.StartsWith(TEXT("/"), ESearchCase::CaseSensitive))
	{
		PluginName.RightChopInline(1);
	}

	int32 FoundIndex = INDEX_NONE;
	if (PluginName.FindChar(TEXT('/'), FoundIndex))
	{
		PluginName.LeftInline(FoundIndex);
	}

	return FindPlugin(PluginName);
}

void FPluginManager::FindPluginsUnderDirectory(const FString& Directory, TArray<FString>& OutPluginFilePaths)
{
	return FindPluginsInDirectory(Directory, OutPluginFilePaths, FPlatformFileManager::Get().GetPlatformFile());
}

TSharedPtr<IPlugin> FPluginManager::FindPluginFromDescriptor(const FPluginReferenceDescriptor& PluginDesc)
{
	const TSharedRef<FPlugin>* Instance = DiscoveredPluginMapUtils::FindPluginInMap_FromDescriptor(AllPlugins, PluginDesc);
	if (Instance == nullptr)
	{
		return TSharedPtr<IPlugin>();
	}
	else
	{
		return TSharedPtr<IPlugin>(*Instance);
	}
}

TSharedPtr<IPlugin> FPluginManager::FindEnabledPlugin(const FStringView Name)
{
	const TSharedPtr<IPlugin> Plugin = FindPlugin(Name);
	if (Plugin.IsValid() && Plugin->IsEnabled())
	{
		return Plugin;
	}
	else
	{
		return TSharedPtr<IPlugin>();
	}
}

TSharedPtr<IPlugin> FPluginManager::FindEnabledPluginFromPath(const FString& PluginPath)
{
	const TSharedPtr<IPlugin> Plugin = FindPluginFromPath(PluginPath);
	if (Plugin.IsValid() && Plugin->IsEnabled())
	{
		return Plugin;
	}
	else
	{
		return TSharedPtr<IPlugin>();
	}
}

TSharedPtr<IPlugin> FPluginManager::FindEnabledPluginFromDescriptor(const FPluginReferenceDescriptor& PluginDesc)
{
	const TSharedPtr<IPlugin> Plugin = FindPluginFromDescriptor(PluginDesc);
	if (Plugin.IsValid() && Plugin->IsEnabled())
	{
		return Plugin;
	}
	else
	{
		return TSharedPtr<IPlugin>();
	}
}

bool FPluginManager::CanEnablePluginInCurrentTarget(const FStringView Name)
{
	if (ConfigureEnabledPlugins())
	{
		const uint32 NameHash = GetTypeHash(Name);
		return PluginsEnabledForTarget.ContainsByHash(NameHash, Name);
	}
	return false;
}

TArray<TSharedRef<IPlugin>> FPluginManager::GetEnabledPlugins()
{
	TArray<TSharedRef<IPlugin>> Plugins;
	Plugins.Reserve(AllPlugins.Num());
	for(FDiscoveredPluginMap::ElementType& PluginPair : AllPlugins)
	{
		TSharedRef<FPlugin>& PossiblePlugin = DiscoveredPluginMapUtils::ResolvePluginFromMapVal(PluginPair.Value);
		if(PossiblePlugin->bEnabled)
		{
			Plugins.Add(PossiblePlugin);
		}
	}
	return Plugins;
}

TArray<TSharedRef<IPlugin>> FPluginManager::GetEnabledPluginsWithContent() const
{
	TArray<TSharedRef<IPlugin>> Plugins;
	Plugins.Reserve(AllPlugins.Num());
	for (const FDiscoveredPluginMap::ElementType& PluginPair : AllPlugins)
	{
		const TSharedRef<FPlugin>& PluginRef = DiscoveredPluginMapUtils::ResolvePluginFromMapVal(PluginPair.Value);
		const FPlugin& Plugin = *PluginRef;
		if (Plugin.IsEnabled() && Plugin.CanContainContent())
		{
			Plugins.Add(PluginRef);
		}
	}
	return Plugins;
}

TArray<TSharedRef<IPlugin>> FPluginManager::GetEnabledPluginsWithVerse() const
{
	TArray<TSharedRef<IPlugin>> Plugins;
	Plugins.Reserve(AllPlugins.Num());
	for (const FDiscoveredPluginMap::ElementType& PluginPair : AllPlugins)
	{
		const TSharedRef<FPlugin>& PluginRef = DiscoveredPluginMapUtils::ResolvePluginFromMapVal(PluginPair.Value);
		const FPlugin& Plugin = *PluginRef;
		if (Plugin.IsEnabled() && (Plugin.CanContainVerse() || (Plugin.CanContainContent() && Plugin.GetDescriptor().bEnableVerseAssetReflection)) && Plugin.IsMounted())
		{
			Plugins.Add(PluginRef);
		}
	}
	return Plugins;
}

TArray<TSharedRef<IPlugin>> FPluginManager::GetEnabledPluginsWithContentOrVerse() const
{
	TArray<TSharedRef<IPlugin>> Plugins;
	Plugins.Reserve(AllPlugins.Num());
	for (const FDiscoveredPluginMap::ElementType& PluginPair : AllPlugins)
	{
		const TSharedRef<FPlugin>& PluginRef = DiscoveredPluginMapUtils::ResolvePluginFromMapVal(PluginPair.Value);
		const FPlugin& Plugin = *PluginRef;
		if (Plugin.IsEnabled() && (Plugin.CanContainContent() || Plugin.CanContainVerse()))
		{
			Plugins.Add(PluginRef);
		}
	}
	return Plugins;
}

TArray<TSharedRef<IPlugin>> FPluginManager::GetDiscoveredPlugins()
{
	TArray<TSharedRef<IPlugin>> Plugins;
	Plugins.Reserve(AllPlugins.Num());

	for (FDiscoveredPluginMap::ElementType& PluginPair : AllPlugins)
	{
		Plugins.Add(DiscoveredPluginMapUtils::ResolvePluginFromMapVal(PluginPair.Value));
	}

	return Plugins;
}

const TSet<FString>& FPluginManager::GetPluginsEnabledAtStartup() const
{
	ensureMsgf(!PluginsEnabledAtStartup.IsEmpty(), TEXT("ConfigureEnabledPlugins hasn't been called yet."));
	return PluginsEnabledAtStartup;
}

#if WITH_EDITOR
const TSet<FString>& FPluginManager::GetBuiltInPluginNames() const
{
	ensure(!BuiltInPluginNames.IsEmpty());
	return BuiltInPluginNames;
}

TSharedPtr<IPlugin> FPluginManager::GetModuleOwnerPlugin(FName ModuleName) const
{
	if (const TSharedRef<IPlugin>* Plugin = ModuleNameToPluginMap.Find(ModuleName))
	{
		return *Plugin;
	}
	return TSharedPtr<IPlugin>();
}
#endif //if WITH_EDITOR

bool FPluginManager::AddPluginSearchPath(const FString& ExtraDiscoveryPath, bool bRefresh)
{
	const FString FullPath = FPaths::ConvertRelativePathToFull(ExtraDiscoveryPath);

	bool bAlreadyExists = false;
	ExternalPluginSources.Add(FExternalPluginPath{ FullPath, EPluginExternalSource::Other });
	PluginDiscoveryPaths.Add(FullPath, &bAlreadyExists);

	if (bRefresh)
	{
		RefreshPluginsList();
	}

	return !bAlreadyExists;
}

bool FPluginManager::RemovePluginSearchPath(const FString& PathToRemove, bool bRefresh)
{
	const FString FullPath = FPaths::ConvertRelativePathToFull(PathToRemove);

	const bool bPathWasPresent = PluginDiscoveryPaths.Contains(FullPath);

	ExternalPluginSources.Remove(FExternalPluginPath{ FullPath, EPluginExternalSource::Other });
	PluginDiscoveryPaths.Remove(FullPath);

	if (bRefresh)
	{
		RefreshPluginsList();
	}

	return bPathWasPresent;
}

const TSet<FString>& FPluginManager::GetAdditionalPluginSearchPaths() const
{
	return PluginDiscoveryPaths;
}

void FPluginManager::GetExternalPluginSources(TSet<FExternalPluginPath>& OutPluginSources) const
{
	OutPluginSources.Append(ExternalPluginSources);

	for (const FString& ProjectPath : IProjectManager::Get().GetAdditionalPluginDirectories())
	{
		OutPluginSources.Add(FExternalPluginPath{ ProjectPath, EPluginExternalSource::ProjectDescriptor });
	}
}

TArray<TSharedRef<IPlugin>> FPluginManager::GetPluginsWithPakFile() const
{
	return PluginsWithPakFile;
}

IPluginManager::FNewPluginMountedEvent& FPluginManager::OnNewPluginCreated()
{
	return NewPluginCreatedEvent;
}

IPluginManager::FNewPluginMountedEvent& FPluginManager::OnNewPluginMounted()
{
	return NewPluginMountedEvent;
}

IPluginManager::FNewPluginMountedEvent& FPluginManager::OnNewPluginContentMounted()
{
	return NewPluginContentMountedEvent;
}

IPluginManager::FNewPluginMountedEvent& FPluginManager::OnPluginEdited()
{
	return PluginEditedEvent;
}

IPluginManager::FNewPluginMountedEvent& FPluginManager::OnPluginUnmounted()
{
	return PluginUnmountedEvent;
}

void FPluginManager::MountNewlyCreatedPlugin(const FString& PluginName)
{
	TSharedPtr<FPlugin> Plugin = FindPluginInstance(PluginName);
	if (Plugin.IsValid())
	{
		MountPluginFromExternalSource(Plugin.ToSharedRef());

		// Notify any listeners that a new plugin has been mounted
		if (NewPluginCreatedEvent.IsBound())
		{
			NewPluginCreatedEvent.Broadcast(*Plugin);
		}
	}
}

bool FPluginManager::MountExplicitlyLoadedPlugin(const FString& PluginName, ELoadingPhase::Type MaxLoadingPhase)
{
	bool bSuccess = false;

	TSharedPtr<FPlugin> Plugin = FindPluginInstance(PluginName);
	if (Plugin.IsValid() && Plugin->Descriptor.bExplicitlyLoaded)
	{
		MountPluginFromExternalSource(Plugin.ToSharedRef(), MaxLoadingPhase);
		bSuccess = true;
	}

	return bSuccess;
}

bool FPluginManager::MountExplicitlyLoadedPlugin_FromFileName(const FString& PluginFileName)
{
	TSharedRef<FPlugin>* DescribedPlugin = DiscoveredPluginMapUtils::FindPluginInMap_FromFileName(AllPlugins, PluginFileName);
	return TryMountExplicitlyLoadedPluginVersion(DescribedPlugin);
}

bool FPluginManager::MountExplicitlyLoadedPlugin_FromDescriptor(const FPluginReferenceDescriptor& PluginDescriptor)
{
	TSharedRef<FPlugin>* DescribedPlugin = DiscoveredPluginMapUtils::FindPluginInMap_FromDescriptor(AllPlugins, PluginDescriptor);
	return TryMountExplicitlyLoadedPluginVersion(DescribedPlugin);
}

bool FPluginManager::MountExplicitlyLoadedPluginLocalizationData(const FString& PluginName, TUniqueFunction<void(bool bLoadedLocalization, const FString& PluginName)>&& Notification)
{
	TSharedPtr<FPlugin> Plugin = FindPluginInstance(PluginName);

	TRACE_CPUPROFILER_EVENT_SCOPE(MountExplicitlyLoadedPluginLocalizationData);
	if (!Plugin.IsValid() || !Plugin->bIsMounted)
	{
		// Does not exist or is not mounted
		UE_LOGF(LogPluginManager, Error, "Cannot mount plugin localization for '%ls' as the plugin is unknown or not mounted. Did you forget to call MountExplicitlyLoadedPlugin?", *PluginName);
		if (Notification)
		{
			Notification(false, PluginName);
		}
		return false;
	}

	if (!Plugin->Descriptor.bExplicitlyLoaded)
	{
		// Not supported
		UE_LOGF(LogPluginManager, Warning, "Cannot mount plugin localization for '%ls' as the plugin isn't explicitly loaded.", *PluginName);
		if (Notification)
		{
			Notification(false, PluginName);
		}
		return false;
	}

	if (Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount > 0)
	{
		// Already loaded
		UE_LOGF(LogPluginManager, Verbose, "Ignoring request to mount plugin localization for '%ls' as it was already referenced (ref-count: %d -> %d).", *PluginName, Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount, Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount + 1);
		++Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount;
		if (Notification)
		{
			Notification(false, PluginName);
		}
		return false;
	}

	if (Plugin->Descriptor.LocalizationTargets.Num() == 0)
	{
		// Nothing to load
		UE_LOGF(LogPluginManager, Verbose, "Ignoring request to mount plugin localization for '%ls' as the plugin has no localization targets defined.", *PluginName);
		if (Notification)
		{
			Notification(false, PluginName);
		}
		return false;
	}

	UE_LOGF(LogPluginManager, Log, "Mounting plugin localization for '%ls'...", *PluginName);
	Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount = 1;

	// Notify that additional localization data should be loaded
	TArray<FString> AdditionalLocResPaths;
	PluginLocalizationUtils::GetLocalizationPathsForPlugin(*Plugin, AdditionalLocResPaths);
	if (Notification)
	{
		FTextLocalizationManager::Get().HandleLocalizationTargetsMounted(AdditionalLocResPaths, [PluginName, Notification = MoveTemp(Notification)]()
		{
			Notification(true, PluginName);
		});
	}
	else
	{
		FTextLocalizationManager::Get().HandleLocalizationTargetsMounted(AdditionalLocResPaths);
	}
	return true;
}

bool FPluginManager::UnmountExplicitlyLoadedPluginLocalizationData(const FString& PluginName, TUniqueFunction<void(bool bUnloadedLocalization, const FString& PluginName)>&& Notification)
{
	TSharedPtr<FPlugin> Plugin = FindPluginInstance(PluginName);

	TRACE_CPUPROFILER_EVENT_SCOPE(UnmountExplicitlyLoadedPluginLocalizationData);
	if (!Plugin.IsValid() || !Plugin->bIsMounted)
	{
		// Does not exist or is not mounted
		UE_LOGF(LogPluginManager, Error, "Cannot unmount plugin localization for '%ls' as the plugin is unknown or not mounted. Did you forget to call MountExplicitlyLoadedPlugin?", *PluginName);
		if (Notification)
		{
			Notification(false, PluginName);
		}
		return false;
	}

	if (!Plugin->Descriptor.bExplicitlyLoaded)
	{
		// Not supported
		UE_LOGF(LogPluginManager, Warning, "Cannot unmount plugin localization for '%ls' as the plugin isn't explicitly loaded.", *PluginName);
		if (Notification)
		{
			Notification(false, PluginName);
		}
		return false;
	}

	if (Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount > 1)
	{
		// Not possible to unload yet
		UE_LOGF(LogPluginManager, Verbose, "Ignoring request to unmount plugin localization for '%ls' as it is still referenced (ref-count: %d -> %d).", *PluginName, Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount, Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount - 1);
		--Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount;
		if (Notification)
		{
			Notification(false, PluginName);
		}
		return false;
	}

	if (Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount == 0)
	{
		// Already unloaded
		UE_LOGF(LogPluginManager, Verbose, "Ignoring request to unmount plugin localization for '%ls' as the localization data was not mounted.", *PluginName);
		if (Notification)
		{
			Notification(false, PluginName);
		}
		return false;
	}

	if (Plugin->Descriptor.LocalizationTargets.Num() == 0)
	{
		// Nothing to load
		UE_LOGF(LogPluginManager, Verbose, "Ignoring request to unmount plugin localization for '%ls' as the plugin has no localization targets defined.", *PluginName);
		if (Notification)
		{
			Notification(false, PluginName);
		}
		return false;
	}

	if (IsEngineExitRequested())
	{
		// Exiting; unloading will be handled by the engine shutdown
		UE_LOGF(LogPluginManager, Verbose, "Ignoring request to unmount plugin localization for '%ls' as the engine is exiting.", *PluginName);
		if (Notification)
		{
			Notification(false, PluginName);
		}
		return false;
	}

	UE_LOGF(LogPluginManager, Log, "Unmounting plugin localization for '%ls'...", *PluginName);
	Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount = 0;

	// Notify that additional localization data should be unloaded
	TArray<FString> AdditionalLocResPaths;
	PluginLocalizationUtils::GetLocalizationPathsForPlugin(*Plugin, AdditionalLocResPaths);
	if (Notification)
	{
		FTextLocalizationManager::Get().HandleLocalizationTargetsUnmounted(AdditionalLocResPaths, [PluginName, Notification = MoveTemp(Notification)]()
		{
			Notification(true, PluginName);
		});
	}
	else
	{
		FTextLocalizationManager::Get().HandleLocalizationTargetsUnmounted(AdditionalLocResPaths);
	}
	return true;
}

bool FPluginManager::TryMountExplicitlyLoadedPluginVersion(TSharedRef<FPlugin>* AllPlugins_PluginPtr)
{
	bool bSuccess = false;

	if (AllPlugins_PluginPtr && (*AllPlugins_PluginPtr)->Descriptor.bExplicitlyLoaded)
	{
		TSharedPtr<FPlugin> CurrentPriorityPlugin = FindPluginInstance((*AllPlugins_PluginPtr)->GetName());
		if (ensure(CurrentPriorityPlugin.IsValid()) && CurrentPriorityPlugin != *AllPlugins_PluginPtr && CurrentPriorityPlugin->IsMounted())
		{
			UE_LOGF(LogPluginManager, Error, "Cannot mount plugin '%ls' with another version (%ls) already mounted.", *(*AllPlugins_PluginPtr)->FileName, *CurrentPriorityPlugin->FileName);
		}
		else
		{
#if WITH_EDITOR
			RemoveFromModuleNameToPluginMap(CurrentPriorityPlugin.ToSharedRef());
			AddToModuleNameToPluginMap(*AllPlugins_PluginPtr);
#endif //if WITH_EDITOR

			// NOTE: This mutates what the passed `DescribedPlugin` pointer is referencing, which is why
			//       we re-assign it to the return value (to keep it referencing the same plugin)
			AllPlugins_PluginPtr = DiscoveredPluginMapUtils::PromotePluginToOfferedVersion(AllPlugins, AllPlugins_PluginPtr);
			if (ensure(AllPlugins_PluginPtr))
			{
				MountPluginFromExternalSource(*AllPlugins_PluginPtr);

				bSuccess = true;
			}
		}
	}

	return bSuccess;
}

void FPluginManager::MountPluginFromExternalSource(const TSharedRef<FPlugin>& Plugin, ELoadingPhase::Type MaxLoadingPhase)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MountPluginFromExternalSource);
	if (GWarn)
	{
		GWarn->BeginSlowTask(FText::Format(LOCTEXT("MountingPluginFiles", "Mounting plugin {0}..."), FText::FromString(Plugin->GetFriendlyName())), /*ShowProgressDialog*/ true, /*bShowCancelButton*/ false);
	}

	// Mark the plugin as enabled
	Plugin->bEnabled = true;
	PluginsEnabledForTarget.Add(Plugin->GetName());

	UE::PluginManager::Private::CheckForDeprecatedPlugin(*Plugin);

	// Mount the plugin content directory
	const bool bHasContentOrVerse = (Plugin->CanContainContent() || Plugin->CanContainVerse()) && ensure(RegisterMountPointDelegate.IsBound());
	if (bHasContentOrVerse)
	{
		if (UE::PluginManager::Private::CoreUObjectPluginHandler)
		{
			UE::PluginManager::Private::CoreUObjectPluginHandler->OnPluginLoad(*Plugin);
		}

		if (NewPluginMountedEvent.IsBound())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR("OnNewPluginMounted");
			NewPluginMountedEvent.Broadcast(*Plugin);
		}

		FString ContentDir = Plugin->GetContentDir();
		{ 
			TRACE_CPUPROFILER_EVENT_SCOPE_STR("RegisterMountPoint");
			RegisterMountPointDelegate.Execute(Plugin->GetMountedAssetPath(), ContentDir);
		}

		// Register this plugin's path with the list of content directories that the editor will search
		if (Plugin->CanContainContent())
		{
			if (FConfigFile* EngineConfigFile = GConfig->Find(GEngineIni))
			{
				EngineConfigFile->AddUniqueToSection(TEXT("Core.System"), "Paths", MoveTemp(ContentDir));
			}
		}
	}

	// Notify that additional localization data should be loaded
	if (!Plugin->Descriptor.bExplicitlyLoaded && Plugin->Descriptor.LocalizationTargets.Num() > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Mount Localization Paths");
		TArray<FString> AdditionalLocResPaths;
		PluginLocalizationUtils::GetLocalizationPathsForPlugin(*Plugin, AdditionalLocResPaths);
		FTextLocalizationManager::Get().HandleLocalizationTargetsMounted(AdditionalLocResPaths);
	}

	// If it's a code module, also load the modules for it
	if (Plugin->Descriptor.Modules.Num() > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Load Plugin Modules");
		// Add the plugin binaries directory
		const FString PluginBinariesPath = FPaths::Combine(*FPaths::GetPath(Plugin->FileName), TEXT("Binaries"), FPlatformProcess::GetBinariesSubdirectory());
		FModuleManager::Get().AddBinariesDirectory(*PluginBinariesPath, Plugin->GetLoadedFrom() == EPluginLoadedFrom::Project);

		// Load all the plugin modules
		for (ELoadingPhase::Type LoadingPhase = (ELoadingPhase::Type)0; LoadingPhase < MaxLoadingPhase; LoadingPhase = (ELoadingPhase::Type)(LoadingPhase + 1))
		{
			if (LoadingPhase != ELoadingPhase::None)
			{
				TryLoadModulesForPlugin(Plugin.Get(), LoadingPhase);
			}
		}
	}

	Plugin->SetIsMounted(true);

	// Notify listeners that the plugin is completely mounted now
	if (bHasContentOrVerse && NewPluginContentMountedEvent.IsBound())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("OnNewPluginContentMounted");
		NewPluginContentMountedEvent.Broadcast(*Plugin);
	}

	if (GWarn)
	{
		GWarn->EndSlowTask();
	}
}

bool FPluginManager::GetPluginDependencies(const FString& PluginName, TArray<FPluginReferenceDescriptor>& PluginDependencies)
{
	const TSharedPtr<FPlugin> Plugin = FindPluginInstance(PluginName);
	if (Plugin.IsValid())
	{
		PluginDependencies = Plugin->Descriptor.Plugins;
		return true;
	}
	return false;
}

bool FPluginManager::GetPluginDependencies_FromFileName(const FString& PluginFileName, TArray<FPluginReferenceDescriptor>& PluginDependencies)
{
	const TSharedRef<FPlugin>* DescribedPlugin = DiscoveredPluginMapUtils::FindPluginInMap_FromFileName(AllPlugins, PluginFileName);
	if (DescribedPlugin)
	{
		PluginDependencies = (*DescribedPlugin)->Descriptor.Plugins;
		return true;
	}
	return false;
}

bool FPluginManager::GetPluginDependencies_FromDescriptor(const FPluginReferenceDescriptor& PluginDescriptor, TArray<FPluginReferenceDescriptor>& PluginDependencies)
{
	const TSharedRef<FPlugin>* DescribedPlugin = DiscoveredPluginMapUtils::FindPluginInMap_FromDescriptor(AllPlugins, PluginDescriptor);
	if (DescribedPlugin)
	{
		PluginDependencies = (*DescribedPlugin)->Descriptor.Plugins;
		return true;
	}
	return false;
}

bool FPluginManager::UnmountExplicitlyLoadedPlugin(const FString& PluginName, FText* OutReason, bool bAllowUnloadCode)
{
	TSharedPtr<FPlugin> Plugin = FindPluginInstance(PluginName);
	TRACE_CPUPROFILER_EVENT_SCOPE(UnmountPluginFromExternalSource);
	if (!Plugin.IsValid() || Plugin->bEnabled == false)
	{
		// Does not exist or is not loaded
		return true;
	}

	if (!Plugin->Descriptor.bExplicitlyLoaded)
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("UnloadPluginNotExplicitlyLoaded", "Plugin was not explicitly loaded");
		}
		return false;
	}

	// Simulate unload of all the plugin modules to gather errors
	// We don't want to actually unload modules until content has been unloaded
	for (ELoadingPhase::Type LoadingPhase = (ELoadingPhase::Type)(ELoadingPhase::Max - 1);
		LoadingPhase >= (ELoadingPhase::Type)0;
		LoadingPhase = (ELoadingPhase::Type)(LoadingPhase - 1))
	{
		if (LoadingPhase != ELoadingPhase::None)
		{
			constexpr bool bSkipUnload = true;
			FText FailureMessage;
			if (!TryUnloadModulesForPlugin(*Plugin, LoadingPhase, &FailureMessage, bSkipUnload))
			{
				if (OutReason)
				{
					*OutReason = MoveTemp(FailureMessage);
				}
				return false;
			}
		}
	}

	// Notify that additional localization data should be unloaded
	if (Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount > 0 && Plugin->Descriptor.LocalizationTargets.Num() > 0 && !IsEngineExitRequested())
	{
		Plugin->ExplicitlyLoadedLocalizationDataMountedRefCount = 0;

		TArray<FString> AdditionalLocResPaths;
		PluginLocalizationUtils::GetLocalizationPathsForPlugin(*Plugin, AdditionalLocResPaths);
		FTextLocalizationManager::Get().HandleLocalizationTargetsUnmounted(AdditionalLocResPaths);
	}

	if ((Plugin->CanContainContent() || Plugin->CanContainVerse()) && ensure(UnRegisterMountPointDelegate.IsBound()))
	{
		// Remove this plugin's path from the list of content directories that the editor will search
		if (Plugin->CanContainContent())
		{
			if (FConfigFile* EngineConfigFile = GConfig->Find(GEngineIni))
			{
				EngineConfigFile->RemoveFromSection(TEXT("Core.System"), "Paths", Plugin->GetContentDir());
			}
		}

		if (PluginUnmountedEvent.IsBound())
		{
			PluginUnmountedEvent.Broadcast(*Plugin);
		}

		UnRegisterMountPointDelegate.Execute(Plugin->GetMountedAssetPath(), Plugin->GetContentDir());

		if (UE::PluginManager::Private::CoreUObjectPluginHandler)
		{
			UE::PluginManager::Private::CoreUObjectPluginHandler->OnPluginUnload(*Plugin);
		}
	}

	// Actually unload all the plugin modules now that content unmount is finished
	for (ELoadingPhase::Type LoadingPhase = (ELoadingPhase::Type)(ELoadingPhase::Max - 1);
		 LoadingPhase >= (ELoadingPhase::Type)0;
		 LoadingPhase = (ELoadingPhase::Type)(LoadingPhase - 1))
	{
		if (LoadingPhase != ELoadingPhase::None)
		{
			verify(TryUnloadModulesForPlugin(*Plugin, LoadingPhase, nullptr, false, bAllowUnloadCode));
		}
	}

	Plugin->SetIsMounted(false);
	Plugin->bEnabled = false;

	return true;
}

void FPluginManager::SuppressPluginUnloadGC()
{
	if (UE::PluginManager::Private::CoreUObjectPluginHandler)
	{
		UE::PluginManager::Private::CoreUObjectPluginHandler->SuppressPluginUnloadGC();
	}
}

void FPluginManager::ResumePluginUnloadGC()
{
	if (UE::PluginManager::Private::CoreUObjectPluginHandler)
	{
		UE::PluginManager::Private::CoreUObjectPluginHandler->ResumePluginUnloadGC();
	}
}

FName FPluginManager::PackageNameFromModuleName(FName ModuleName)
{
	FName Result = ModuleName;
	for (FDiscoveredPluginMap::TIterator Iter(AllPlugins); Iter; ++Iter)
	{
		const TSharedRef<FPlugin>& Plugin = DiscoveredPluginMapUtils::ResolvePluginFromMapVal(Iter.Value());
		const TArray<FModuleDescriptor>& Modules = Plugin->Descriptor.Modules;
		for (int Idx = 0; Idx < Modules.Num(); Idx++)
		{
			const FModuleDescriptor& Descriptor = Modules[Idx];
			if (Descriptor.Name == ModuleName)
			{
				UE_LOGF(LogPluginManager, Verbose, "Module %ls belongs to Plugin %ls and we assume that is the name of the package with the UObjects is /Script/%ls", *ModuleName.ToString(), *Plugin->Name, *Plugin->Name);
				return FName(*Plugin->Name);
			}
		}
	}
	return Result;
}

void FPluginManager::EnumerateVersePathMountPoints(const UE::Core::FVersePath& VersePath, TFunctionRef<bool(FStringView /*MountedAssetPath*/, FStringView /*LeafPath*/)> Callback)
{
	// Can't do anything with an empty vpath
	if (!VersePath.IsValid())
	{
		return;
	}

	const FStringView VersePathView = VersePath.AsStringView();

	for (const TPair<FString, TArray<TSharedRef<FPlugin>>>& NamePluginPair : AllPlugins)
	{
		const TSharedRef<FPlugin>& Plugin = DiscoveredPluginMapUtils::ResolvePluginFromMapVal(NamePluginPair.Value);

		if (Plugin->IsMounted() && !Plugin->GetVersePath().IsEmpty())
		{
			UE::Core::FVersePath PluginVersePath;
			if (UE::Core::FVersePath::TryMake(PluginVersePath, Plugin->GetVersePath()))
			{
				FStringView LeafPath;
				if (PluginVersePath.IsBaseOf(VersePath, &LeafPath))
				{
					if (!Callback(Plugin->GetMountedAssetPath(), LeafPath))
					{
						break;
					}
				}
			}
		}
	}
}

#if WITH_EDITOR
void FPluginManager::AddToModuleNameToPluginMap(const TSharedRef<FPlugin>& Plugin)
{
	if (!Plugin->Descriptor.bIsPluginExtension)
	{
		for (const FModuleDescriptor& Module : Plugin->Descriptor.Modules)
		{
			if (const TSharedRef<IPlugin>* FoundPlugin = ModuleNameToPluginMap.Find(Module.Name))
			{
				if (*FoundPlugin != Plugin)
				{
					UE_LOGF(LogPluginManager, Display, "Module %ls from plugin %ls is already associated with plugin %ls (maybe because the plugin should be using bIsPluginExtension)", *Module.Name.ToString(), *Plugin->GetName(), *(*FoundPlugin)->GetName());
				}
			}
			else
			{
				ModuleNameToPluginMap.Add(Module.Name, Plugin);
			}
		}
	}
}

void FPluginManager::RemoveFromModuleNameToPluginMap(const TSharedRef<FPlugin>& Plugin)
{
	if (!Plugin->Descriptor.bIsPluginExtension)
	{
		for (const FModuleDescriptor& Module : Plugin->Descriptor.Modules)
		{
			TSharedRef<IPlugin> RemovedPlugin(Plugin);
			if (ModuleNameToPluginMap.RemoveAndCopyValue(Module.Name, RemovedPlugin))
			{
				ensure(RemovedPlugin == Plugin);
			}
		}
	}
}
#endif //if WITH_EDITOR

void FPluginManager::GetExplanationForUnavailablePackage(const FName& UnavailablePackageName, FStringBuilderBase& InOutExplanation)
{
	FString UnavailablePackageNameString = UnavailablePackageName.ToString();
	FPlugin* Plugin = static_cast<FPlugin*>(FindPluginFromPath(UnavailablePackageNameString).Get());

	if (Plugin != nullptr)
	{
		if (!Plugin->IsMounted())
		{
			InOutExplanation.Appendf(TEXT("Skipped package %s was expected to be found in plugin %s, however that plugin is not mounted. [ExplicitlyLoaded = %s, Enabled = %s]."),
				*UnavailablePackageNameString,
				*Plugin->GetFriendlyName(),
				Plugin->GetDescriptor().bExplicitlyLoaded ? TEXT("TRUE") : TEXT("FALSE"),
				Plugin->IsEnabled() ? TEXT("TRUE") : TEXT("FALSE"));
		}
	}
	else
	{
		// If we don't have a valid plugin, we have nothing to add
	}

	// Give higher level systems a chance to provide further information
	GetExplanationForUnavailablePackageWithPluginInfoDelegate().Broadcast(UnavailablePackageNameString, Plugin, InOutExplanation);
}

IPluginManager::FGetExplanationForUnavailablePackageWithPluginInfoDelegate& FPluginManager::GetExplanationForUnavailablePackageWithPluginInfoDelegate()
{
	static FGetExplanationForUnavailablePackageWithPluginInfoDelegate Delegate;
	return Delegate;
}


#undef LOCTEXT_NAMESPACE
