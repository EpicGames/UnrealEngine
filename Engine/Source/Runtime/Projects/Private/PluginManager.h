// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PluginDescriptor.h"
#include "Interfaces/IPluginManager.h"

struct FProjectDescriptor;
class FJsonObject;

/**
 * Instance of a plugin in memory
 */
class FPlugin final : public IPlugin
{
public:
	/** The name of the plugin */
	FString Name;

	/** The filename that the plugin was loaded from */
	FString FileName;

	/** The filenames from which any child plugins that have been merged were loaded */
	TArray<FString> PluginExtensionFileNameList;

	/** The plugin's settings */
	FPluginDescriptor Descriptor;

	/** Type of plugin */
	EPluginType Type;

	/** True if the plugin is marked as enabled */
	bool bEnabled : 1;

	/** 
	 * True if this plugin has been mounted/loaded. A 'mounted' plugin is one 
	 * whose content has been registered, and modules set to load.
	 * NOTE: Usually plugins are 'enabled' and 'mounted' at the same time, with 
	 *       the exception of 'ExplicitlyLoaded' plugins (which can be set  
	 *       'enabled' but not mounted until much later)
	 */
	bool bIsMounted : 1;

	/**
	 * >0 if an explicitly loaded plugin has also mounted its localization data.
	 * @note Unused for non-explicitly loaded plugins.
	 */
	uint8 ExplicitlyLoadedLocalizationDataMountedRefCount = 0;

	/**
	 * FPlugin constructor
	 */
	FPlugin(const FString &FileName, const FPluginDescriptor& InDescriptor, EPluginType InType);

	/**
	 * Destructor.
	 */
	virtual ~FPlugin();

	/* IPlugin interface */
	virtual const FString& GetName() const override
	{
		return Name;
	}

	virtual const FString& GetFriendlyName() const override
	{
		return GetDescriptor().FriendlyName.IsEmpty() ? GetName() : GetDescriptor().FriendlyName;
	}

	virtual const FString& GetDescriptorFileName() const override
	{
		return FileName;
	}

	virtual FString GetBaseDir() const override;
	virtual TArray<FString> GetExtensionBaseDirs() const override;
	virtual FString GetContentDir() const override;
	virtual FString GetMountedAssetPath() const override;

	virtual bool IsEnabled() const override
	{
		return bEnabled;
	}

	virtual bool IsEnabledByDefault(bool bAllowEnginePluginsEnabledByDefault) const override;

	virtual bool IsMounted() const override
	{
		return bIsMounted;
	}

	virtual bool IsHidden() const override
	{
		return Descriptor.bIsHidden;
	}

	virtual bool CanContainContent() const override
	{
		return Descriptor.bCanContainContent;
	}

	virtual bool CanContainVerse() const override
	{
		return Descriptor.bCanContainVerse;
	}

	virtual const FString& GetVersePath() const override
	{
		return Descriptor.VersePath;
	}

	virtual TOptional<uint32> GetVerseVersion() const override
	{
		return Descriptor.VerseVersion;
	}

	virtual const FString& GetDeprecatedEngineVersion() const override
	{
		return Descriptor.DeprecatedEngineVersion;
	}
		
	virtual bool IsSceneGraphEnabled() const override
	{
		return Descriptor.bEnableSceneGraph;
	}

	virtual bool IsSceneGraphAnimationEnabled() const override
	{
		return Descriptor.bEnableSceneGraphAnimation;
	}

	virtual bool IsRootPlugin() const override
	{
		return Descriptor.bNoCode && Descriptor.bNoGameFeatureContent;
	}

	virtual EPluginType GetType() const override
	{
		return Type;
	}

	virtual EPluginLoadedFrom GetLoadedFrom() const override;
	virtual const FPluginDescriptor& GetDescriptor() const override;
	virtual bool UpdateDescriptor(const FPluginDescriptor& NewDescriptor, FText& OutFailReason) override;
#if WITH_EDITOR
	virtual const TSharedPtr<FJsonObject>& GetDescriptorJson() override;
#endif // WITH_EDITOR

	void SetIsMounted(bool bInIsMounted) { bIsMounted = bInIsMounted; }
};

/**
 * FPluginManager manages available code and content extensions (both loaded and not loaded.)
 */
class FPluginManager final : public IPluginManager
{
public:
	/** Constructor */
	FPluginManager();

	/** Destructor */
	~FPluginManager();

	/** IPluginManager interface */
	virtual void RefreshPluginsList() override;
	virtual bool AddToPluginsList(const FString& PluginFilename, FText* OutFailReason = nullptr) override;
	virtual bool RemoveFromPluginsList(const FString& PluginFilename, FText* OutFailReason = nullptr) override;
	virtual bool LoadModulesForEnabledPlugins( const ELoadingPhase::Type LoadingPhase ) override;
	virtual FLoadingModulesForPhaseEvent& OnLoadingPhaseComplete() override;
	virtual ELoadingPhase::Type GetLastCompletedLoadingPhase() const override;
	virtual void GetLocalizationPathsForEnabledPlugins( TArray<FString>& OutLocResPaths ) override;
	virtual void SetRegisterMountPointDelegate( const FRegisterMountPointDelegate& Delegate ) override;
	virtual void SetUnRegisterMountPointDelegate( const FRegisterMountPointDelegate& Delegate ) override;
	virtual bool AreRequiredPluginsAvailable() override;
	virtual FGetExplanationForUnavailablePackageWithPluginInfoDelegate& GetExplanationForUnavailablePackageWithPluginInfoDelegate() override;

	virtual void SuppressPluginUnloadGC() override;
	virtual void ResumePluginUnloadGC() override;

#if !IS_MONOLITHIC
	virtual bool CheckModuleCompatibility(TArray<FString>& OutIncompatibleModules, TArray<FString>& OutIncompatibleEngineModules) override;
#endif
	virtual TSharedPtr<IPlugin> FindPlugin(const FStringView Name) override;
	virtual TSharedPtr<IPlugin> FindPlugin(const ANSICHAR* Name) override
	{
		FString NameString(Name);
		return FindPlugin(FStringView(NameString));
	}
	virtual TSharedPtr<IPlugin> FindPluginFromPath(const FString& PluginPath) override;
	virtual TSharedPtr<IPlugin> FindPluginFromDescriptor(const FPluginReferenceDescriptor& PluginDesc) override;

	virtual TSharedPtr<IPlugin> FindEnabledPlugin(const FStringView Name) override;
	virtual TSharedPtr<IPlugin> FindEnabledPlugin(const ANSICHAR* Name) override
	{
		FString NameString(Name);
		return FindEnabledPlugin(FStringView(NameString));
	}
	virtual TSharedPtr<IPlugin> FindEnabledPluginFromPath(const FString& PluginPath) override;
	virtual TSharedPtr<IPlugin> FindEnabledPluginFromDescriptor(const FPluginReferenceDescriptor& PluginDesc) override;

	virtual bool CanEnablePluginInCurrentTarget(const FStringView Name) override;
	virtual bool CanEnablePluginInCurrentTarget(const ANSICHAR* Name) override
	{
		FString NameString(Name);
		return CanEnablePluginInCurrentTarget(FStringView(NameString));
	}

	virtual void FindPluginsUnderDirectory(const FString& Directory, TArray<FString>& OutPluginFilePaths) override;

	virtual TArray<TSharedRef<IPlugin>> GetEnabledPlugins() override;
	virtual TArray<TSharedRef<IPlugin>> GetEnabledPluginsWithContent() const override;
	virtual TArray<TSharedRef<IPlugin>> GetEnabledPluginsWithVerse() const override;
	virtual TArray<TSharedRef<IPlugin>> GetEnabledPluginsWithContentOrVerse() const override;
	virtual TArray<TSharedRef<IPlugin>> GetDiscoveredPlugins() override;

	virtual const TSet<FString>& GetPluginsEnabledAtStartup() const override;
#if WITH_EDITOR
	virtual const TSet<FString>& GetBuiltInPluginNames() const override;
	virtual TSharedPtr<IPlugin> GetModuleOwnerPlugin(FName ModuleName) const override;
#endif //WITH_EDITOR

	virtual bool AddPluginSearchPath(const FString& ExtraDiscoveryPath, bool bRefresh = true) override;
	virtual bool RemovePluginSearchPath(const FString& PathToRemove, bool bRefresh = true) override;
	virtual const TSet<FString>& GetAdditionalPluginSearchPaths() const override;
	virtual void GetExternalPluginSources(TSet<FExternalPluginPath>& OutPluginSources) const override;
	virtual TArray<TSharedRef<IPlugin>> GetPluginsWithPakFile() const override;
	virtual FNewPluginMountedEvent& OnNewPluginCreated() override;
	virtual FNewPluginMountedEvent& OnNewPluginMounted() override;
	virtual FNewPluginMountedEvent& OnNewPluginContentMounted() override;
	virtual FNewPluginMountedEvent& OnPluginEdited() override;
	virtual FNewPluginMountedEvent& OnPluginUnmounted() override;
	virtual void MountNewlyCreatedPlugin(const FString& PluginName) override;
	virtual bool MountExplicitlyLoadedPlugin(const FString& PluginName, ELoadingPhase::Type MaxLoadingPhase = ELoadingPhase::Max) override;
	virtual bool MountExplicitlyLoadedPlugin_FromFileName(const FString& PluginFileName) override;
	virtual bool MountExplicitlyLoadedPlugin_FromDescriptor(const FPluginReferenceDescriptor& PluginDescriptor) override;
	virtual bool MountExplicitlyLoadedPluginLocalizationData(const FString& PluginName, TUniqueFunction<void(bool bLoadedLocalization, const FString& PluginName)>&& Notification = nullptr) override;
	virtual bool UnmountExplicitlyLoadedPluginLocalizationData(const FString& PluginName, TUniqueFunction<void(bool bUnloadedLocalization, const FString& PluginName)>&& Notification = nullptr) override;
	virtual bool UnmountExplicitlyLoadedPlugin(const FString& PluginName, FText* OutReason) override { return UnmountExplicitlyLoadedPlugin(PluginName, OutReason, true); }
	virtual bool UnmountExplicitlyLoadedPlugin(const FString& PluginName, FText* OutReason, bool bAllowUnloadCode) override;
	virtual bool GetPluginDependencies(const FString& PluginName, TArray<FPluginReferenceDescriptor>& PluginDependencies) override;
	virtual bool GetPluginDependencies_FromFileName(const FString& PluginFileName, TArray<FPluginReferenceDescriptor>& PluginDependencies) override;
	virtual bool GetPluginDependencies_FromDescriptor(const FPluginReferenceDescriptor& PluginDescriptor, TArray<FPluginReferenceDescriptor>& PluginDependencies) override;
	virtual FName PackageNameFromModuleName(FName ModuleName) override;
	virtual void EnumerateVersePathMountPoints(const UE::Core::FVersePath& VersePath, TFunctionRef<bool(FStringView /*MountedAssetPath*/, FStringView /*LeafPath*/)> Callback) override;
	virtual bool RequiresTempTargetForCodePlugin(const FProjectDescriptor* ProjectDescriptor, const FString& Platform, EBuildConfiguration Configuration, EBuildTargetType TargetType, FText& OutReason) override;

	virtual void SetBinariesRootDirectories(const FString& EngineBinariesRootDir, const FString& ProjectBinariesRootDir) override;
	virtual void SetPreloadBinaries() override;
	virtual bool GetPreloadBinaries() override;
	virtual FLeakedPackagesFoundEvent& OnLeakedPackagesFound() override;

	void GetExplanationForUnavailablePackage(const FName& UnavailablePackageName, FStringBuilderBase& InOutExplanation) override;

private:
	using FDiscoveredPluginMap = TMap<FString, TArray<TSharedRef<FPlugin>>>;

	struct FConfigurePluginResultInfo;

	/** Searches for all plugins on disk and builds up the array of plugin objects.  Doesn't load any plugins. 
	    This is called when the plugin manager singleton is first accessed. */
	void DiscoverAllPlugins();
	
	/**
	 * Helper functions just to split up the massive size of ConfigureEnabledPlugins()
	 */
	struct FPluginDiscoveryContext
	{
		// Set of all the plugins which have been enabled
		TMap<FString, FPlugin*> EnabledPlugins;

		// Keep a set of all the plugin names that have been configured. We read configuration data from different places, but only configure a plugin from the first place that it's referenced.
		TSet<FString> ConfiguredPluginNames;

		// Keep the list of newly available localization targets
		TArray<FString> AdditionalLocResPaths;

#if UE_BUILD_SHIPPING
		// const to ensure it to stays empty
		const TSet<FString> AllowedOptionalDependencies;
#else
		// Set of all the plugin names that are allowed to be enabled for a plugin with optional dependencies. Only read in Editor
		TSet<FString> AllowedOptionalDependencies;
#endif

	};
	void FindCompileTimePlugins(FPluginDiscoveryContext& Context);
	bool FindCommandLinePlugins(FPluginDiscoveryContext& Context);
	bool FindTargetPlugins(FPluginDiscoveryContext& Context);
	bool FindProgramPlugins(FPluginDiscoveryContext& Context);
	void MarkEnabledPlugins(FPluginDiscoveryContext& Context);
	void ProcessEnabledPlugins(FPluginDiscoveryContext& Context);
	void MountContentPlugins(FPluginDiscoveryContext& Context);

	/** Reads all the plugin descriptors */
	static void ReadAllPlugins(FDiscoveredPluginMap& Plugins, const TSet<FString>& ExtraSearchPaths,
		TArray<FString>* OutPluginSources=nullptr);

	/** Reads all the plugin descriptors from disk */
	static void ReadPluginsInDirectory(const FString& PluginsDirectory, const EPluginType Type, FDiscoveredPluginMap& Plugins, TArray<TSharedRef<FPlugin>>& ChildPlugins);

	/** Creates a FPlugin object and adds it to the given map */
	static void CreatePluginObject(const FString& FileName, const FPluginDescriptor& Descriptor, const EPluginType Type, FDiscoveredPluginMap& Plugins, TArray<TSharedRef<FPlugin>>& ChildPlugins);

	/** Finds all the plugin descriptors underneath a given directory */
	static void FindPluginsInDirectory(const FString& PluginsDirectory, TArray<FString>& FileNames, IPlatformFile& PlatformFile);

	/** Finds all the plugin manifests in a given directory */
	static void FindPluginManifestsInDirectory(const FString& PluginManifestDirectory, TArray<FString>& FileNames);

	/** Gets all the code plugins that are enabled for a content only project */
	static bool GetCodePluginsForProject(const FProjectDescriptor* ProjectDescriptor, const FString& Platform, EBuildConfiguration Configuration, EBuildTargetType TargetType, FDiscoveredPluginMap& AllPlugins, TSet<FString>& CodePluginNames, const TSet<FString>& AllowedOptionalPluginReferences, FConfigurePluginResultInfo& OutResultInfo);

	/** Sets the bPluginEnabled flag on all plugins found from DiscoverAllPlugins that are enabled in config */
	bool ConfigureEnabledPlugins();

	/** Adds a single enabled plugin, and all its dependencies. If AllowedOptionalDependencies is not empty, only enable optional dependencies that are listed */
	bool ConfigureEnabledPluginForCurrentTarget(const FPluginReferenceDescriptor& FirstReference,
		TMap<FString, FPlugin*>& EnabledPlugins, FStringView SourceOfPluginRequest, const TSet<FString>& AllowedOptionalDependencies);

	/** Adds a single enabled plugin and all its dependencies. If AllowedOptionalDependencies is not empty, only enable optional dependencies that are listed */
	static bool ConfigureEnabledPluginForTarget(const FPluginReferenceDescriptor& FirstReference, const FProjectDescriptor* ProjectDescriptor, const FString& TargetName, const FString& Platform, EBuildConfiguration Configuration, EBuildTargetType TargetType, bool bLoadPluginsForTargetPlatforms, FDiscoveredPluginMap& AllPlugins, TMap<FString, FPlugin*>& EnabledPlugins, const TSet<FString>& AllowedOptionalDependencies, FConfigurePluginResultInfo& OutResultInfo);

	/** Prompts the user to download a missing plugin from the given URL */
	static bool PromptToDownloadPlugin(const FString& PluginName, const FString& MarketplaceURL);

	/** Prompts the user to disable a plugin */
	static bool PromptToDisableSealedPlugin(const FString& PluginName, const FString& SealedPluginName);

	/** Prompts the user to disable a plugin */
	static bool PromptToDisableDisalowedPlugin(const FString& PluginName, const FString& DisallowedPluginName);

	/** Prompts the user to disable a plugin */
	static bool PromptToDisableMissingPlugin(const FString& PluginName, const FString& MissingPluginName);

	/** Prompts the user to disable a plugin */
	static bool PromptToDisableIncompatiblePlugin(const FString& PluginName, const FString& IncompatiblePluginName);

	/** Prompts the user to disable a plugin */
	static bool PromptToDisablePlugin(const FText& Caption, const FText& Message, const FString& PluginName);

	/** Checks whether a plugin is compatible with the current engine version */
	static bool IsPluginCompatible(const FPlugin& Plugin);

	/** Prompts the user to disable a plugin */
	static bool PromptToLoadIncompatiblePlugin(const FPlugin& Plugin);

	/** Attempt to load all the modules for the given plugin */
	bool TryLoadModulesForPlugin(const FPlugin& Plugin, const ELoadingPhase::Type LoadingPhase) const;

	/** Attempt to unload all the modules for the given plugin */
	bool TryUnloadModulesForPlugin(const FPlugin& Plugin, const ELoadingPhase::Type LoadingPhase, FText* OutFailureMessage = nullptr, bool bSkipUnload = false, bool bAllowUnloadCode = true) const;

	/** Gets the instance of a given plugin */
	TSharedPtr<FPlugin> FindPluginInstance(const FString& Name);

	/** 
	 * Attempts to mount a spectific plugin version. Can fail if the plugin isn't marked 'ExplicitlyLoaded',
	 * or if there's a different version already mounted.
	 * 
	 * NOTE: It's expected that `AllPlugins_PluginPtr` directly addresses an entry in `AllPlugins` 
	 *       (so that it can reorder the versions in `AllPlugins` since that's how we track the mounted/choice version).
	 */
	bool TryMountExplicitlyLoadedPluginVersion(TSharedRef<FPlugin>* AllPlugins_PluginPtr);

	/** Mounts a plugin that was requested to be mounted from external code (either by MountNewlyCreatedPlugin or MountExplicitlyLoadedPlugin) */
	void MountPluginFromExternalSource(const TSharedRef<FPlugin>& Plugin, ELoadingPhase::Type MaxLoadingPhase = ELoadingPhase::Max);

#if WITH_EDITOR
	void AddToModuleNameToPluginMap(const TSharedRef<FPlugin>& Plugin);
	void RemoveFromModuleNameToPluginMap(const TSharedRef<FPlugin>& Plugin);
#endif //if WITH_EDITOR

private:
	/** All of the plugins that we know about */
	FDiscoveredPluginMap AllPlugins;

	/** Plugins that need to be configured to see if they should be enabled */
	TSet<FString> PluginsToConfigure;

	/** Set of plugins enabled for the current target. Plugins in list might not be currently enabled. */
	TSet<FString> PluginsEnabledForTarget;

	/** Set of plugins which were compiled for the target and enabled at startup. */
	TSet<FString> PluginsEnabledAtStartup;

#if WITH_EDITOR
	/** Names of built-in plugins */
	TSet<FString> BuiltInPluginNames;

	TMap<FName, TSharedRef<IPlugin>> ModuleNameToPluginMap;
#endif //if WITH_EDITOR

	TArray<TSharedRef<IPlugin>> PluginsWithPakFile;

	/** Delegate for mounting content paths.  Bound by FPackageName code in CoreUObject, so that we can access
	    content path mounting functionality from Core. */
	FRegisterMountPointDelegate RegisterMountPointDelegate;

	/** Delegate for unmounting content paths.  Bound by FPackageName code in CoreUObject, so that we can access
	    content path unmounting functionality from Core. */
	FRegisterMountPointDelegate UnRegisterMountPointDelegate;

	/** Set if all the required plugins are available */
	bool bHaveAllRequiredPlugins = false;

	/** Set if we were asked to load all plugins via the command line */
	bool bAllPluginsEnabledViaCommandLine = false;

	bool bPreloadedBinaries = false;

	/** List of additional directory paths to search for plugins within */
	TSet<FExternalPluginPath> ExternalPluginSources;

	/** Projection of ExternalPluginSources paths only; maintained for backward compatibility */
	TSet<FString> PluginDiscoveryPaths;

	/** Callback for notifications that a new plugin was mounted */
	FNewPluginMountedEvent NewPluginCreatedEvent;
	FNewPluginMountedEvent NewPluginMountedEvent;
	FNewPluginMountedEvent NewPluginContentMountedEvent;
	FNewPluginMountedEvent PluginEditedEvent;
	FNewPluginMountedEvent PluginUnmountedEvent;

	/** Callback for notifications that a loading phase was completed */
	FLoadingModulesForPhaseEvent LoadingPhaseCompleteEvent;

	/** The highest LoadingPhase that has so far completed */
	ELoadingPhase::Type LastCompletedLoadingPhase = ELoadingPhase::None;

	FString EngineBinariesRootDir;
	FString ProjectBinariesRootDir;

	/** The callback that is called from within the PluginHandler when leaked packages are found */
	FLeakedPackagesFoundEvent LeakedPackagesFoundEvent;
};


