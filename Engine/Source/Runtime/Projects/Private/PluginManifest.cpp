// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginManifest.h"
#include "Misc/FileHelper.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "JsonUtils/JsonConversion.h"
#include "Misc/Paths.h"
#include "RapidJsonPluginLoading.h"

#define LOCTEXT_NAMESPACE "PluginManifest"

namespace PluginManifest
{
	void DumpContents(const TArray<FPluginManifestEntry>& Contents, const FString& Filename)
	{
		TArray<FString> Lines;

		for (const FPluginManifestEntry& Entry : Contents)
		{
			FString Line;
			Entry.Descriptor.Write(Line);

			Line = Entry.File + TEXT("\n") + Line;

			Lines.Add(MoveTemp(Line));
		}

		Lines.Sort();

		FString DumpFilename = FPaths::ProjectSavedDir() / Filename;
		FFileHelper::SaveStringArrayToFile(Lines, *DumpFilename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
	}
}

TOptional<FText> UE::Projects::Private::Read(UE::Json::FConstObject Object, FPluginManifest& Result)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UE::Projects::Private::Read FPluginManifest");
	using namespace UE::Json;

	TOptional<FConstArray> JsonContents = GetArrayField(Object, TEXT("Contents"));
	if (!JsonContents.IsSet())
	{
		return LOCTEXT("PluginDescriptorMissingContentsArray", "No 'Contents' array found");
	}

	Result.Contents.Reserve(JsonContents->Size());

	for (const FValue& JsonEntry : *JsonContents)
	{
		if (!JsonEntry.IsObject())
		{
			continue;
		}

		FPluginManifestEntry& ManifestEntry = Result.Contents.Emplace_GetRef();

		FConstObject JsonEntryObject = JsonEntry.GetObject();
		TOptional<FConstObject> JsonDescriptor = GetObjectField(JsonEntryObject, TEXT("Descriptor"));

		if (!JsonDescriptor.IsSet())
		{
			return FText::Format(LOCTEXT("PluginDescriptorIndexMissingDescriptorField", "Plugin descriptor index '{0}' missing 'Descriptor' field"), Result.Contents.Num()-1);
		}

		if (!TryGetStringField(JsonEntryObject, TEXT("File"), ManifestEntry.File))
		{
			return FText::Format(LOCTEXT("PluginDescriptorIndexMissingFileField", "Plugin descriptor index '{0}' missing 'File' field"), Result.Contents.Num()-1);
		}

		if (TOptional<FText> ReadError = UE::Projects::Private::Read(*JsonDescriptor, ManifestEntry.Descriptor))
		{
			return FText::Format(LOCTEXT("PluginDescriptorEntryParseError", "Failed to parse plugin descriptor entry for file '{0}': {1}"), FText::FromString(ManifestEntry.File), *ReadError);
		}
	}

	return {};
}

bool FPluginManifest::Load(const FString& FileName, FText& OutFailReason)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FPluginManifest::Load");
	// Read the file to a string
	FString FileContents;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("LoadFileToString");
		if (!FFileHelper::LoadFileToString(FileContents, *FileName))
		{
			OutFailReason = FText::Format(LOCTEXT("FailedToLoadDescriptorFile", "Failed to open descriptor file '{0}'"), FText::FromString(FileName));
			return false;
		}
	}

	{
		SCOPED_BOOT_TIMING("FPluginManifest::Load RapidJson")

		// grab the characters so we can parse insitu safely (FString API says adding non-terminating zeros can be unsafe so just take the data)
		TArray<TCHAR> Characters = MoveTemp(FileContents.GetCharArray());
		TValueOrError<UE::Json::FDocument, UE::Json::FParseError> Document = UE::Json::ParseInPlace(Characters);

		if (Document.HasError())
		{
			// have to re-parse the string because it was destructively altered
			ensure(FFileHelper::LoadFileToString(FileContents, *FileName));
			OutFailReason = FText::Format(LOCTEXT("FailedToReadDescriptorFile", "Failed to read file. {0}"), FText::FromString(Document.GetError().CreateMessage(FileContents)));
			
			return false;
		}

		TOptional<UE::Json::FConstObject> RootObject = UE::Json::GetRootObject(Document.GetValue());
		if (RootObject.IsSet())
		{
			if (TOptional<FText> Result = UE::Projects::Private::Read(*RootObject, *this))
			{
				OutFailReason = *Result;
				return false;
			}
		}

		return true;
	}
}

bool FPluginManifest::Read(const FJsonObject& Object, FText& OutFailReason)
{
	return UE::Projects::Private::ReadFromDefaultJson(Object, *this, &OutFailReason);
}

#undef LOCTEXT_NAMESPACE
