// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginReferenceDescriptor.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "JsonUtils/JsonObjectArrayUpdater.h"
#include "ProjectDescriptor.h"
#include "RapidJsonPluginLoading.h"
#include "JsonUtils/JsonConversion.h"

#define LOCTEXT_NAMESPACE "PluginDescriptor"

namespace PluginReferenceDescriptor
{
	FString GetPluginRefKey(const FPluginReferenceDescriptor& PluginRef)
	{
		return PluginRef.Name;
	}

	bool TryGetPluginRefJsonObjectKey(const FJsonObject& JsonObject, FString& OutKey)
	{
		return JsonObject.TryGetStringField(TEXT("Name"), OutKey);
	}

	void UpdatePluginRefJsonObject(const FPluginReferenceDescriptor& PluginRef, FJsonObject& JsonObject)
	{
		PluginRef.UpdateJson(JsonObject);
	}
}

FPluginReferenceDescriptor::FPluginReferenceDescriptor(FString InName, bool bInEnabled)
	: Name(MoveTemp(InName))
	, bEnabled(bInEnabled)
	, bOptional(false)
	, bActivate(false)
	, bHasExplicitPlatforms(false)
{ }


bool FPluginReferenceDescriptor::IsEnabledForPlatform( const FString& Platform ) const
{
	// If it's not enabled at all, return false
	if(!bEnabled)
	{
		return false;
	}

	// If there is a list of allowed platform platforms, and this isn't one of them, return false
	if( (bHasExplicitPlatforms || PlatformAllowList.Num() > 0) && !PlatformAllowList.Contains(Platform))
	{
		return false;
	}

	// If this platform is denied, also return false
	if(PlatformDenyList.Contains(Platform))
	{
		return false;
	}

	return true;
}

bool FPluginReferenceDescriptor::IsEnabledForTarget(EBuildTargetType TargetType) const
{
    // If it's not enabled at all, return false
    if (!bEnabled)
    {
        return false;
    }

    // If there is a list of allowed targets, and this isn't one of them, return false
    if (TargetAllowList.Num() > 0 && !TargetAllowList.Contains(TargetType))
    {
        return false;
    }

    // If this platform is denied, also return false
    if (TargetDenyList.Contains(TargetType))
    {
        return false;
    }

    return true;
}

bool FPluginReferenceDescriptor::IsEnabledForTargetConfiguration(EBuildConfiguration Configuration) const
{
	// If it's not enabled at all, return false
	if (!bEnabled)
	{
		return false;
	}

	// If there is a list of allowed target configurations, and this isn't one of them, return false
	if (TargetConfigurationAllowList.Num() > 0 && !TargetConfigurationAllowList.Contains(Configuration))
	{
		return false;
	}

	// If this target configuration is denied, also return false
	if (TargetConfigurationDenyList.Contains(Configuration))
	{
		return false;
	}

	return true;
}

bool FPluginReferenceDescriptor::IsSupportedTargetPlatform(const FString& Platform) const
{
	if (bHasExplicitPlatforms)
	{
		return SupportedTargetPlatforms.Contains(Platform);
	}
	else
	{
		return SupportedTargetPlatforms.Num() == 0 || SupportedTargetPlatforms.Contains(Platform);
	}
}

TOptional<FText> UE::Projects::Private::Read(UE::Json::FConstObject Object, FPluginReferenceDescriptor& Result)
{
	using namespace UE::Json;

#if WITH_EDITOR
	Result.CachedJson = ConvertRapidJsonToSharedJsonObject(Object);
	Result.AdditionalFieldsToWrite.Reset();
#endif // WITH_EDITOR

	// Get the name
	if (!TryGetStringField(Object, TEXT("Name"), Result.Name))
	{
		return LOCTEXT("PluginReferenceWithoutName", "Plugin references must have a 'Name' field");
	}

	// Get the enabled field
	if (!TryGetBoolField(Object, TEXT("Enabled"), Result.bEnabled))
	{
		return LOCTEXT("PluginReferenceWithoutEnabled", "Plugin references must have an 'Enabled' field");
	}

	// Read the optional field
	TryGetBoolField(Object, TEXT("Optional"), Result.bOptional);

	TryGetBoolField(Object, TEXT("Activate"), Result.bActivate);

	// Read the metadata for users that don't have the plugin installed
	TryGetStringField(Object, TEXT("Description"), Result.Description);
	TryGetStringField(Object, TEXT("MarketplaceURL"), Result.MarketplaceURL);

	// Get the platform lists
	TryGetStringArrayFieldWithDeprecatedFallback(Object, TEXT("PlatformAllowList"), TEXT("WhitelistPlatforms"), /*out*/ Result.PlatformAllowList);
	TryGetStringArrayFieldWithDeprecatedFallback(Object, TEXT("PlatformDenyList"), TEXT("BlacklistPlatforms"), /*out*/ Result.PlatformDenyList);

	// Get the target configuration lists
	TryGetEnumArrayFieldWithDeprecatedFallback(Object, TEXT("TargetConfigurationAllowList"), TEXT("WhitelistTargetConfigurations"), /*out*/ Result.TargetConfigurationAllowList);
	TryGetEnumArrayFieldWithDeprecatedFallback(Object, TEXT("TargetConfigurationDenyList"), TEXT("BlacklistTargetConfigurations"), /*out*/ Result.TargetConfigurationDenyList);

	// Get the target lists
	TryGetEnumArrayFieldWithDeprecatedFallback(Object, TEXT("TargetAllowList"), TEXT("WhitelistTargets"), /*out*/ Result.TargetAllowList);
	TryGetEnumArrayFieldWithDeprecatedFallback(Object, TEXT("TargetDenyList"), TEXT("BlacklistTargets"), /*out*/ Result.TargetDenyList);

	// Get the target list for explicitly loading targets
	TryGetEnumArrayField(Object, TEXT("TargetsToExplicitlyLoad"), /*out*/ Result.TargetsToExplicitlyLoad);

	// Get the supported platform list
	TryGetStringArrayField(Object, TEXT("SupportedTargetPlatforms"), Result.SupportedTargetPlatforms);
	TryGetBoolField(Object, TEXT("HasExplicitPlatforms"), Result.bHasExplicitPlatforms);

	int32 ReadVersion;
	if (TryGetNumberField(Object, TEXT("Version"), ReadVersion))
	{
		Result.RequestedVersion = ReadVersion;
		if (!Result.bEnabled)
		{
			return LOCTEXT("PluginReferenceDisabledWithVersion", "Plugin references cannot be used to disable explicit versions. Remove the 'Version' field when 'Enabled' is false.");
			}
		}

	return {};
}

bool FPluginReferenceDescriptor::Read(const TSharedRef<FJsonObject>& Object, FText* OutFailReason /*= nullptr*/)
{
	return UE::Projects::Private::ReadFromDefaultJson(*Object, *this, OutFailReason);
}

bool FPluginReferenceDescriptor::Read(const FJsonObject& Object, FText* OutFailReason /*= nullptr*/, TSharedPtr<FJsonObject> ObjectPtr /*= nullptr*/)
{
	return UE::Projects::Private::ReadFromDefaultJson(Object, *this, OutFailReason);
}

bool FPluginReferenceDescriptor::Read(const FJsonObject& Object, FText& OutFailReason, TSharedPtr<FJsonObject> ObjectPtr /*= nullptr*/)
{
	return UE::Projects::Private::ReadFromDefaultJson(Object, *this, &OutFailReason);
}

bool FPluginReferenceDescriptor::ReadArray(const FJsonObject& Object, const TCHAR* Name, TArray<FPluginReferenceDescriptor>& OutPlugins, FText* OutFailReason /*= nullptr*/)
{
	return UE::Projects::Private::ReadArrayFromDefaultJson(Object, Name, OutPlugins, OutFailReason);
}

bool FPluginReferenceDescriptor::ReadArray(const FJsonObject& Object, const TCHAR* Name, TArray<FPluginReferenceDescriptor>& OutPlugins, FText& OutFailReason)
{
	return UE::Projects::Private::ReadArrayFromDefaultJson(Object, Name, OutPlugins, &OutFailReason);
}

void FPluginReferenceDescriptor::Write(TJsonWriter<>& Writer) const
{
	TSharedPtr<FJsonObject> PluginRefJsonObject = MakeShared<FJsonObject>();

#if WITH_EDITOR
	if (CachedJson.IsValid())
	{
		FJsonObject::Duplicate(/*Source=*/ CachedJson, /*Dest=*/ PluginRefJsonObject);
	}
#endif //if WITH_EDITOR

	UpdateJson(*PluginRefJsonObject);

	FJsonSerializer::Serialize(PluginRefJsonObject.ToSharedRef(), Writer);
}

void FPluginReferenceDescriptor::UpdateJson(FJsonObject& JsonObject) const
{
	JsonObject.SetStringField(TEXT("Name"), Name);
	JsonObject.SetBoolField(TEXT("Enabled"), bEnabled);

	if (bActivate)
	{
		JsonObject.SetBoolField(TEXT("Activate"), bActivate);
	}
	else
	{
		JsonObject.RemoveField(TEXT("Activate"));
	}

	if (bEnabled && bOptional)
	{
		JsonObject.SetBoolField(TEXT("Optional"), bOptional);
	}
	else
	{
		JsonObject.RemoveField(TEXT("Optional"));
	}

	if (Description.Len() > 0)
	{
		JsonObject.SetStringField(TEXT("Description"), Description);
	}
	else
	{
		JsonObject.RemoveField(TEXT("Description"));
	}

	if (MarketplaceURL.Len() > 0)
	{
		JsonObject.SetStringField(TEXT("MarketplaceURL"), MarketplaceURL);
	}
	else
	{
		JsonObject.RemoveField(TEXT("MarketplaceURL"));
	}

	if (PlatformAllowList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> PlatformAllowListValues;
		for (const FString& Platform : PlatformAllowList)
		{
			PlatformAllowListValues.Add(MakeShareable(new FJsonValueString(Platform)));
		}
		JsonObject.SetArrayField(TEXT("PlatformAllowList"), PlatformAllowListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("PlatformAllowList"));
	}

	if (PlatformDenyList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> PlatformDenyListValues;
		for (const FString& Platform : PlatformDenyList)
		{
			PlatformDenyListValues.Add(MakeShareable(new FJsonValueString(Platform)));
		}
		JsonObject.SetArrayField(TEXT("PlatformDenyList"), PlatformDenyListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("PlatformDenyList"));
	}

	if (TargetConfigurationAllowList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> TargetConfigurationAllowListValues;
		for (EBuildConfiguration Config : TargetConfigurationAllowList)
		{
			TargetConfigurationAllowListValues.Add(MakeShareable(new FJsonValueString(LexToString(Config))));
		}
		JsonObject.SetArrayField(TEXT("TargetConfigurationAllowList"), TargetConfigurationAllowListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("TargetConfigurationAllowList"));
	}

	if (TargetConfigurationDenyList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> TargetConfigurationDenyListValues;
		for (EBuildConfiguration Config : TargetConfigurationDenyList)
		{
			TargetConfigurationDenyListValues.Add(MakeShareable(new FJsonValueString(LexToString(Config))));
		}
		JsonObject.SetArrayField(TEXT("TargetConfigurationDenyList"), TargetConfigurationDenyListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("TargetConfigurationDenyList"));
	}

	if (TargetAllowList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> TargetAllowListValues;
		for (EBuildTargetType Target : TargetAllowList)
		{
			TargetAllowListValues.Add(MakeShareable(new FJsonValueString(LexToString(Target))));
		}
		JsonObject.SetArrayField(TEXT("TargetAllowList"), TargetAllowListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("TargetAllowList"));
	}

	if (TargetDenyList.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> TargetDenyListValues;
		for (EBuildTargetType Target : TargetDenyList)
		{
			TargetDenyListValues.Add(MakeShareable(new FJsonValueString(LexToString(Target))));
		}
		JsonObject.SetArrayField(TEXT("TargetDenyList"), TargetDenyListValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("TargetDenyList"));
	}

	if (TargetsToExplicitlyLoad.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> TargetsToExplicitlyLoadValues;
		for (EBuildTargetType Target : TargetsToExplicitlyLoad)
		{
			TargetsToExplicitlyLoadValues.Add(MakeShareable(new FJsonValueString(LexToString(Target))));
		}
		JsonObject.SetArrayField(TEXT("TargetsToExplicitlyLoad"), TargetsToExplicitlyLoadValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("TargetsToExplicitlyLoad"));
	}

	if (SupportedTargetPlatforms.Num() > 0)
	{
		TArray<TSharedPtr<FJsonValue>> SupportedTargetPlatformValues;
		for (const FString& SupportedTargetPlatform : SupportedTargetPlatforms)
		{
			SupportedTargetPlatformValues.Add(MakeShareable(new FJsonValueString(SupportedTargetPlatform)));
		}
		JsonObject.SetArrayField(TEXT("SupportedTargetPlatforms"), SupportedTargetPlatformValues);
	}
	else
	{
		JsonObject.RemoveField(TEXT("SupportedTargetPlatforms"));
	}

	if (bHasExplicitPlatforms)
	{
		JsonObject.SetBoolField(TEXT("HasExplicitPlatforms"), bHasExplicitPlatforms);
	}
	else
	{
		JsonObject.RemoveField(TEXT("HasExplicitPlatforms"));
	}

	if (bEnabled && RequestedVersion.IsSet())
	{
		JsonObject.SetNumberField(TEXT("Version"), RequestedVersion.GetValue());
	}
	else
	{
		JsonObject.RemoveField(TEXT("Version"));
	}

	// Remove deprecated fields
	JsonObject.RemoveField(TEXT("WhitelistPlatforms"));
	JsonObject.RemoveField(TEXT("BlacklistPlatforms"));
	JsonObject.RemoveField(TEXT("WhitelistTargetConfigurations"));
	JsonObject.RemoveField(TEXT("BlacklistTargetConfigurations"));
	JsonObject.RemoveField(TEXT("WhitelistTargets"));
	JsonObject.RemoveField(TEXT("BlacklistTargets"));

#if WITH_EDITOR
	for (const auto& KVP : AdditionalFieldsToWrite)
	{
		JsonObject.SetField(KVP.Key, FJsonValue::Duplicate(KVP.Value));
	}
#endif //if WITH_EDITOR
}

void FPluginReferenceDescriptor::WriteArray(TJsonWriter<>& Writer, const TCHAR* ArrayName, const TArray<FPluginReferenceDescriptor>& Plugins)
{
	if (Plugins.Num() > 0)
	{
		Writer.WriteArrayStart(ArrayName);

		for (const FPluginReferenceDescriptor& PluginRef : Plugins)
		{
			PluginRef.Write(Writer);
		}

		Writer.WriteArrayEnd();
	}
}

void FPluginReferenceDescriptor::UpdateArray(FJsonObject& JsonObject, const TCHAR* ArrayName, const TArray<FPluginReferenceDescriptor>& Plugins)
{
	typedef FJsonObjectArrayUpdater<FPluginReferenceDescriptor, FString> FPluginRefJsonArrayUpdater;

	FPluginRefJsonArrayUpdater::Execute(
		JsonObject, ArrayName, Plugins, 
		FPluginRefJsonArrayUpdater::FGetElementKey::CreateStatic(PluginReferenceDescriptor::GetPluginRefKey),
		FPluginRefJsonArrayUpdater::FTryGetJsonObjectKey::CreateStatic(PluginReferenceDescriptor::TryGetPluginRefJsonObjectKey),
		FPluginRefJsonArrayUpdater::FUpdateJsonObject::CreateStatic(PluginReferenceDescriptor::UpdatePluginRefJsonObject),
		FPluginRefJsonArrayUpdater::FSortArray::CreateLambda([&Plugins](TArray<TSharedPtr<FJsonValue>>& NewJsonValues) {
			// Sort the json array to match the same order as the plugin array. Without the sort, new entries are appended at the end
			for (int32 StartIndex = 0; StartIndex < Plugins.Num(); ++StartIndex)
			{
				const FString PluginRefKey = PluginReferenceDescriptor::GetPluginRefKey(Plugins[StartIndex]);
				for (int32 Index = StartIndex; Index < NewJsonValues.Num(); ++Index)
				{
					const TSharedPtr<FJsonObject>* ExistingJsonValueAsObject;
					if (NewJsonValues[Index]->TryGetObject(ExistingJsonValueAsObject))
					{
						FString ElementKey;
						if (PluginReferenceDescriptor::TryGetPluginRefJsonObjectKey(**ExistingJsonValueAsObject, ElementKey))
						{
							if (ElementKey == PluginRefKey)
							{
								NewJsonValues.Swap(StartIndex, Index);
								break;
							}
						}
					}
				}
			}
		})
	);
}

#if WITH_EDITOR
bool FPluginReferenceDescriptor::GetAdditionalStringField(const FString& Key, FString& OutValue) const
{
	if (const TSharedPtr<FJsonValue>* ValuePtr = AdditionalFieldsToWrite.Find(Key))
	{
		const TSharedPtr<FJsonValue>& Value = *ValuePtr;
		if (Value && Value->Type == EJson::String)
		{
			OutValue = Value->AsString();
			return true;
		}
	}

	if (CachedJson)
	{
		if (TSharedPtr<FJsonValue> Value = CachedJson->TryGetField(Key))
		{
			if (Value->Type == EJson::String)
			{
				OutValue = Value->AsString();
				return true;
			}
		}
	}

	return false;
}
#endif //if WITH_EDITOR

#undef LOCTEXT_NAMESPACE
