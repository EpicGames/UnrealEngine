// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProjectDescriptor.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Interfaces/IProjectManager.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "JsonUtils/JsonConversion.h"
#include "RapidJsonPluginLoading.h"

#define LOCTEXT_NAMESPACE "ProjectDescriptor"

FProjectDescriptor::FProjectDescriptor()
{
	FileVersion = EProjectDescriptorVersion::Latest;
	EpicSampleNameHash = 0;
	bIsEnterpriseProject = false;
	bDisableEnginePluginsByDefault = false;
}

void FProjectDescriptor::Sign(const FString& FilePath)
{
	EpicSampleNameHash = GetTypeHash(FPaths::GetCleanFilename(FilePath));
}

bool FProjectDescriptor::IsSigned(const FString& FilePath) const
{
	return EpicSampleNameHash == GetTypeHash(FPaths::GetCleanFilename(FilePath));
}

int32 FProjectDescriptor::FindPluginReferenceIndex(const FString& PluginName) const
{
	for(int32 Idx = 0; Idx < Plugins.Num(); Idx++)
	{
		if(Plugins[Idx].Name == PluginName)
		{
			return Idx;
		}
	}
	return INDEX_NONE;
}

void FProjectDescriptor::UpdateSupportedTargetPlatforms(const FName& InPlatformName, bool bIsSupported)
{
	if ( bIsSupported )
	{
		TargetPlatforms.AddUnique(InPlatformName);
	}
	else
	{
		TargetPlatforms.Remove(InPlatformName);
	}
}

bool FProjectDescriptor::Load(const FString& FileName, FText& OutFailReason)
{
	SCOPED_BOOT_TIMING("FProjectDescriptor::Load")

	// Read the file to a string
	FString FileContents;
	if (!FFileHelper::LoadFileToString(FileContents, *FileName))
	{
		// Do not try to localize these messages, a missing project file usually indicates missing data and an
		// access to ICU will result in failure that's harder to diagnose
		OutFailReason = FText::FromString(FString::Printf(TEXT("Failed to open descriptor file %s"), *FileName));
		return false;
	}

	// grab the characters so we can parse insitu safely (FString API says adding non-terminating zeros can be unsafe so just take the data)
	TArray<TCHAR> Characters = MoveTemp(FileContents.GetCharArray());
	TValueOrError<UE::Json::FDocument, UE::Json::FParseError> ParseResult = UE::Json::ParseInPlace(Characters);

	if (ParseResult.HasError())
	{
		// have to re-load the file because it was destructively altered
		ensure(FFileHelper::LoadFileToString(FileContents, *FileName));

		// Do not try to localize these messages, a missing project file usually indicates missing data and an
		// access to ICU will result in failure that's harder to diagnose
		OutFailReason = FText::FromString(FString::Printf(TEXT("Failed to read file %s: %s"), *FileName, *ParseResult.GetError().CreateMessage(FileContents)));
		return false;
	}

	TOptional<UE::Json::FConstObject> RootObject = UE::Json::GetRootObject(ParseResult.GetValue());
	if (!RootObject.IsSet())
	{
		return false;
	}

	return Read(*RootObject, FPaths::GetPath(FileName), OutFailReason);
}

bool FProjectDescriptor::Read(const UE::Json::FConstObject Object, const FString& PathToProject, FText& OutFailReason)
{
	SCOPED_BOOT_TIMING("FProjectDescriptor::Read")

	using namespace UE::Json;
	using namespace UE::Projects::Private;

	// Read the file version
	int32 FileVersionInt32 = (int32)EProjectDescriptorVersion::Invalid;
	if(!TryGetNumberField(Object, TEXT("FileVersion"), FileVersionInt32))
	{
		if(!TryGetNumberField(Object, TEXT("ProjectFileVersion"), FileVersionInt32))
		{
			OutFailReason = LOCTEXT("InvalidProjectFileVersion", "File does not have a valid 'FileVersion' number.");
			return false;
		}
	}

	// Check that it's within range
	FileVersion = (EProjectDescriptorVersion::Type)FileVersionInt32;
	if ( FileVersion <= EProjectDescriptorVersion::Invalid || FileVersion > EProjectDescriptorVersion::Latest )
	{
		FText ReadVersionText = FText::FromString( FString::Printf( TEXT( "%d" ), (int32)FileVersion ) );
		FText LatestVersionText = FText::FromString( FString::Printf( TEXT( "%d" ), (int32)EProjectDescriptorVersion::Latest ) );
		OutFailReason = FText::Format( LOCTEXT("ProjectFileVersionTooLarge", "File appears to be in a newer version ({0}) of the file format that we can load (max version: {1})."), ReadVersionText, LatestVersionText);
		return false;
	}

	// Read simple fields
	TryGetStringField(Object, TEXT("EngineAssociation"), EngineAssociation);
	TryGetStringField(Object, TEXT("Category"), Category);
	TryGetStringField(Object, TEXT("Description"), Description);
	TryGetBoolField(Object, TEXT("Enterprise"), bIsEnterpriseProject);
	TryGetBoolField(Object, TEXT("DisableEnginePluginsByDefault"), bDisableEnginePluginsByDefault);

#if WITH_EDITOR
	ModuleNamesCache.Reset();
#endif

	if (TOptional<FText> ReadModulesError = ReadArray(Object, TEXT("Modules"), Modules))
	{
		OutFailReason = *ReadModulesError;
		return false;
	}

	if (TOptional<FText> ReadPluginsError = ReadArray(Object, TEXT("Plugins"), Plugins))
	{
		OutFailReason = *ReadPluginsError;
		return false;
	}

	// Read the list of additional plugin directories to scan
	TArray<FString> AdditionalPluginDirectoriesValue;
	if (TryGetStringArrayField(Object, TEXT("AdditionalPluginDirectories"), AdditionalPluginDirectoriesValue))
	{
#if WITH_EDITOR || (IS_PROGRAM && WITH_PLUGIN_SUPPORT)
		for (FString& AdditionalPluginDir : AdditionalPluginDirectoriesValue)
		{
				if (FPaths::IsRelative(AdditionalPluginDir))
				{
					AdditionalPluginDir = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*(PathToProject / AdditionalPluginDir));
				}
				AddPluginDirectory(AdditionalPluginDir);
			}

#endif //if WITH_EDITOR
		// If this is a packaged build and there are additional directories, they need to be remapped to the packaged location
		if (FPlatformProperties::RequiresCookedData() && AdditionalPluginDirectoriesValue.Num() > 0)
		{
			AdditionalPluginDirectories.Empty();
			FString RemappedDir = FPaths::ProjectDir() / TEXT("../RemappedPlugins/");
			if (FPaths::IsRelative(RemappedDir))
			{
				RemappedDir = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*RemappedDir);
			}
			AddPluginDirectory(RemappedDir);
		}
	}

#if WITH_EDITOR
	// Read the list of additional root directories to scan
	TArray<FString> AdditionalRootDirectoriesValue;
	if (TryGetStringArrayField(Object, TEXT("AdditionalRootDirectories"), AdditionalRootDirectoriesValue))
	{
		for (FString& AdditionalRoot : AdditionalRootDirectoriesValue)
		{
				if (FPaths::IsRelative(AdditionalRoot))
				{
					AdditionalRoot = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*(PathToProject / AdditionalRoot));
				}
				AddRootDirectory(AdditionalRoot);
			}
		}
#endif //if WITH_EDITOR

	// Read the target platforms
	TryGetStringArrayField(Object, TEXT("TargetPlatforms"), TargetPlatforms);

	// check if the project has directories for extended platforms, and assume support if it does
	// however, if there were no platforms already listed, then all platforms are supported, and we don't
	// want to add a platform or two here, because then _only_ those platforms will be supported
	// (empty TargetPlatforms array means _all_ platforms are supported)
	if (TargetPlatforms.Num() > 0)
	{
		TArray<FString> ExtendedPlatforms;
		IFileManager::Get().IterateDirectory(*(FPaths::Combine(PathToProject, TEXT("Platforms"))), [&ExtendedPlatforms](const TCHAR* InFilenameOrDirectory, const bool bInIsDirectory) -> bool
		{
			if (bInIsDirectory)
			{
				FString LastDirectory = FPaths::GetBaseFilename(FString(InFilenameOrDirectory));
				ExtendedPlatforms.Emplace(LastDirectory);
			}
			return true;
		});

		const TMap<FName, FDataDrivenPlatformInfo>& AllPlatformInfos = FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos();
		for (const FString& ExtendedPlatform : ExtendedPlatforms)
		{
			FName PlatformName(*ExtendedPlatform);
			if (AllPlatformInfos.Contains(PlatformName))
			{
				TargetPlatforms.AddUnique(PlatformName);
			}
		}
	}

	// Get the sample name hash
	TryGetNumberField(Object, TEXT("EpicSampleNameHash"), EpicSampleNameHash);

	// Read the custom build steps
	PreBuildSteps = ReadCustomBuildSteps(Object, TEXT("PreBuildSteps"));
	PostBuildSteps = ReadCustomBuildSteps(Object, TEXT("PostBuildSteps"));

	return true;
}

bool FProjectDescriptor::Read(const FJsonObject& Object, const FString& PathToProject, FText& OutFailReason)
{
	return UE::Projects::Private::ReadFromDefaultJsonHelper(
		Object,
		[&PathToProject, &OutFailReason, this](UE::Json::FConstObject Object)
		{
			return this->Read(Object, PathToProject, OutFailReason);
		}
	);
}

bool FProjectDescriptor::Save(const FString& FileName, FText& OutFailReason)
{
	if (IProjectManager::Get().IsSuppressingProjectFileWrite())
	{
		OutFailReason = FText::Format( LOCTEXT("FailedToWriteOutputFileSuppressed", "Failed to write output file '{0}'. Project file saving is suppressed."), FText::FromString(FileName) );
		return false;
	}

	// Write the contents of the descriptor to a string. Make sure the writer is destroyed so that the contents are flushed to the string.
	FString Text;
	TSharedRef< TJsonWriter<> > Writer = TJsonWriterFactory<>::Create(&Text);
	Write(Writer.Get(), FPaths::GetPath(FileName));
	Writer->Close();

	// Save it to a file
	if ( FFileHelper::SaveStringToFile(Text, *FileName) )
	{
		return true;
	}
	else
	{
		OutFailReason = FText::Format( LOCTEXT("FailedToWriteOutputFile", "Failed to write output file '{0}'. Perhaps the file is Read-Only?"), FText::FromString(FileName) );
		return false;
	}
}

void FProjectDescriptor::Write(TJsonWriter<>& Writer, const FString& PathToProject) const
{
	Writer.WriteObjectStart();

	// Write all the simple fields
	Writer.WriteValue(TEXT("FileVersion"), EProjectDescriptorVersion::Latest);
	Writer.WriteValue(TEXT("EngineAssociation"), EngineAssociation);
	Writer.WriteValue(TEXT("Category"), Category);
	Writer.WriteValue(TEXT("Description"), Description);

	if (bDisableEnginePluginsByDefault)
	{
		Writer.WriteValue(TEXT("DisableEnginePluginsByDefault"), bDisableEnginePluginsByDefault);
	}

	// Write the enterprise flag
	if (bIsEnterpriseProject)
	{
		Writer.WriteValue(TEXT("Enterprise"), bIsEnterpriseProject);
	}

	// Write the module list
	FModuleDescriptor::WriteArray(Writer, TEXT("Modules"), Modules);

	// Write the plugin list
	FPluginReferenceDescriptor::WriteArray(Writer, TEXT("Plugins"), Plugins);

	// Write out the additional plugin directories to scan
	if (AdditionalPluginDirectories.Num() > 0)
	{
		Writer.WriteArrayStart(TEXT("AdditionalPluginDirectories"));
		for (const FString& Dir : AdditionalPluginDirectories)
		{
			// Convert to relative path if possible before writing it out
			Writer.WriteValue(MakePathRelativeToProject(Dir, PathToProject));
		}
		Writer.WriteArrayEnd();
	}

	// Write out the additional root directories to scan
	if (AdditionalRootDirectories.Num() > 0)
	{
		Writer.WriteArrayStart(TEXT("AdditionalRootDirectories"));
		for (const FString& Dir : AdditionalRootDirectories)
		{
			// Convert to relative path if possible before writing it out
			Writer.WriteValue(MakePathRelativeToProject(Dir, PathToProject));
		}
		Writer.WriteArrayEnd();
	}

	// Write the target platforms
	if(TargetPlatforms.Num() > 0)
	{
		Writer.WriteArrayStart(TEXT("TargetPlatforms"));
		for(int Idx = 0; Idx < TargetPlatforms.Num(); Idx++)
		{
			Writer.WriteValue(TargetPlatforms[Idx].ToString());
		}
		Writer.WriteArrayEnd();
	}

	// If it's a signed sample, write the name hash
	if(EpicSampleNameHash != 0)
	{
		Writer.WriteValue(TEXT("EpicSampleNameHash"), FString::Printf(TEXT("%u"), EpicSampleNameHash));
	}

	// Write the custom build steps
	if(!PreBuildSteps.IsEmpty())
	{
		PreBuildSteps.Write(Writer, TEXT("PreBuildSteps"));
	}
	if(!PostBuildSteps.IsEmpty())
	{
		PostBuildSteps.Write(Writer, TEXT("PostBuildSteps"));
	}

	Writer.WriteObjectEnd();
}

FString FProjectDescriptor::GetExtension()
{
	static const FString ProjectExtension("uproject");
	return ProjectExtension;
}

/** @return the path relative to this project if possible */
const FString FProjectDescriptor::MakePathRelativeToProject(const FString& Dir, const FString& PathToProject) const
{
	FString ProjectDir = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*(PathToProject / TEXT("")));
	FPaths::MakePlatformFilename(ProjectDir);
	FString ModifiedDir(Dir);
	FPaths::MakePathRelativeTo(ModifiedDir, *ProjectDir);
	return ModifiedDir;
}

bool FProjectDescriptor::AddPluginDirectory(const FString& Dir)
{
#if WITH_EDITOR
	if (!ensureMsgf(!FPaths::IsRelative(Dir), TEXT("Cannot add plugin directory: %s is not an absolute path"), *Dir))
	{
		return false;
	}
#endif
	if (Dir.StartsWith(IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FPaths::ProjectPluginsDir())) ||
		Dir.StartsWith(IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FPaths::EnginePluginsDir())))
	{
		return false;
	}

	if (!AdditionalPluginDirectories.Contains(Dir))
	{
		AdditionalPluginDirectories.Add(Dir);
		return true;
	}
	return false;
}

bool FProjectDescriptor::RemovePluginDirectory(const FString& Dir)
{
	if (!ensureMsgf(!FPaths::IsRelative(Dir), TEXT("Cannot remove plugin directory: %s is not an absolute path"), *Dir))
	{
		return false;
	}
	return AdditionalPluginDirectories.RemoveSingle(Dir) > 0;
}

bool FProjectDescriptor::AddRootDirectory(const FString& Dir)
{
	if (!ensureMsgf(!FPaths::IsRelative(Dir), TEXT("Cannot add root directory: %s is not an absolute path"), *Dir))
	{
		return false;
	}
	if (Dir.StartsWith(IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FPaths::EngineDir())) ||
		Dir.StartsWith(IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FPaths::ProjectDir())))
	{
		return false;
	}

	if (!AdditionalRootDirectories.Contains(Dir))
	{
		AdditionalRootDirectories.Add(Dir);
		return true;
	}
	return false;
}

bool FProjectDescriptor::RemoveRootDirectory(const FString& Dir)
{
	if (!ensureMsgf(!FPaths::IsRelative(Dir), TEXT("Cannot remove root directory: %s is not an absolute path"), *Dir))
	{
		return false;
	}
	return AdditionalRootDirectories.RemoveSingle(Dir) > 0;
}

#if WITH_EDITOR
bool FProjectDescriptor::HasModule(FName ModuleName) const
{
	if (ModuleNamesCache.Num() != Modules.Num())
	{
		ModuleNamesCache.Reset();
		ModuleNamesCache.Reserve(Modules.Num());
		for (const FModuleDescriptor& Module : Modules)
		{
			ModuleNamesCache.Add(Module.Name);
		}
		ensure(ModuleNamesCache.Num() == Modules.Num());
	}
	return ModuleNamesCache.Contains(ModuleName);
}
#endif //if WITH_EDITOR

#undef LOCTEXT_NAMESPACE
