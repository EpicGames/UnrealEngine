// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProjectEditorRecords.h"
#include "HAL/PlatformProcess.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/Paths.h"
#include "Misc/App.h"
#include "HAL/ConsoleManager.h"
#include "Async/TaskGraphInterfaces.h"

const FString FProjectEditorRecord::ProjectsProperty = TEXT("Projects");
const FString FProjectEditorRecord::SubProjectProperty = TEXT("SubProjects");
const FString FProjectEditorRecord::EngineLocationProperty = TEXT("EngineLocation");
const FString FProjectEditorRecord::BaseDirProperty = TEXT("BaseDir");
const FString FProjectEditorRecord::TimestampProperty = TEXT("LastAccessed");
const FString FProjectEditorRecord::EpicAppProperty = TEXT("EpicApp");
FGraphEventRef FProjectEditorRecord::AsyncUpdateTask = nullptr;


namespace ProjectEditorRecords
{
	// Lazy initialized in GetFileLocation if not set
	FString ProjectEngineLocationFile;
	static FAutoConsoleVariableRef CVarProjectEngineLocationFile(
		TEXT("r.Editor.ProjectEditorRecordsFile"),
		ProjectEngineLocationFile,
		TEXT("The path of the Project - Engine Location record file.")
	);
}

FProjectEditorRecord FProjectEditorRecord::Load()
{	
	FProjectEditorRecord Records;
	FString FileContents;
	if (!FFileHelper::LoadFileToString(FileContents, *GetFileLocation()))
	{
		Records.ProjectEditorJson = MakeShared<FJsonObject>();
		return Records;
	}

	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileContents);
	if (!FJsonSerializer::Deserialize(Reader, Records.ProjectEditorJson) || !Records.ProjectEditorJson.IsValid())
	{
		Records.ProjectEditorJson = MakeShared<FJsonObject>();
	}

	return Records;
}

bool FProjectEditorRecord::Save() 
{
	PruneOldEntries(ProjectEditorJson);
	FString FileContents;

	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&FileContents);
	if (!FJsonSerializer::Serialize(ProjectEditorJson.ToSharedRef(), Writer))
	{
		return false;
	}

	if (!FFileHelper::SaveStringToFile(FileContents, *GetFileLocation()))
	{
		return false;
	}

	return true;
}

const TSharedPtr<FJsonObject> FProjectEditorRecord::FindOrAddProperty(const FString& InProperty)
{
	if (GIsEditor && !IsRunningCommandlet())
	{
		if (!ProjectEditorJson->HasTypedField(InProperty, EJson::Object))
		{
			ProjectEditorJson->SetObjectField(InProperty, MakeShared<FJsonObject>());
		}

		const TSharedPtr<FJsonObject>& Field = ProjectEditorJson->GetObjectField(InProperty);
		Field->SetStringField(FProjectEditorRecord::TimestampProperty, FDateTime::UtcNow().ToString());
		return Field;
	}

	return nullptr;
}

TSharedPtr<FJsonObject> FProjectEditorRecord::MakeDefaultProperties()
{
	TSharedPtr<FJsonObject> JsonProperties = MakeShared<FJsonObject>();
	JsonProperties->SetStringField(FProjectEditorRecord::EngineLocationProperty, FPlatformProcess::ExecutablePath());
	JsonProperties->SetStringField(FProjectEditorRecord::BaseDirProperty, FPlatformProcess::BaseDir());
	JsonProperties->SetStringField(FProjectEditorRecord::TimestampProperty, FDateTime::UtcNow().ToString());
	return JsonProperties;
}

void FProjectEditorRecord::QueueUpdate(TUniqueFunction<void(FProjectEditorRecord&)>&& InUpdateFunction)
{
	if (GIsEditor && !IsRunningCommandlet())
	{
		TUniqueFunction<void()> Task = [Update = MoveTemp(InUpdateFunction)]()
		{
			FSystemWideCriticalSection SystemMutex(TEXT("ProjectEditorRecords"), FTimespan::FromMinutes(1));
			FProjectEditorRecord AssociationFile = FProjectEditorRecord::Load();
			Update(AssociationFile);
			AssociationFile.Save();
		};

		if (FTaskGraphInterface::IsRunning())
		{
			if (AsyncUpdateTask)
			{
				AsyncUpdateTask = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(Task), TStatId(), AsyncUpdateTask);
			}
			else
			{
				AsyncUpdateTask = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(Task), TStatId());
			}
		}
		else
		{
			Task();
		}
	}
}
 
void FProjectEditorRecord::TearDown()
{
	if (AsyncUpdateTask)
	{
		if (FTaskGraphInterface::IsRunning())
		{
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(AsyncUpdateTask);
		}
		AsyncUpdateTask.SafeRelease();
	}
}

void FProjectEditorRecord::PruneOldEntries(const TSharedPtr<FJsonObject>& JsonObject)
{
	TArray<FString> KeysToRemove;
	for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Field : JsonObject->Values)
	{
		// Non-object fields are left alone, top level should always be objects with a LastAccessed field that we can check
		if (Field.Value->Type != EJson::Object)
		{
			continue;
		}

		TSharedPtr<FJsonObject>* ProjectInfo;
		FString TimestampStr;
		FDateTime Timestamp;

		if (!Field.Value->TryGetObject(ProjectInfo) 
		|| !(*ProjectInfo)->TryGetStringField(FProjectEditorRecord::TimestampProperty, TimestampStr) 
		|| !FDateTime::Parse(TimestampStr, Timestamp) 
		|| (Timestamp - FDateTime::UtcNow()).GetTotalDays() > DaysToKeepRecords)
		{
			// If it's not valid, mark it for pruning
			KeysToRemove.Add(FString(Field.Key));
		}
		else
		{
			// If it's valid, search child objects recursiveness for pruning
			PruneOldEntries(Field.Value->AsObject());
		}
	}

	for (const FString& Key : KeysToRemove)
	{
		ProjectEditorJson->RemoveField(Key);
	}
}

const FString FProjectEditorRecord::GetFileLocation()
{
	FString FileLocation = ProjectEditorRecords::CVarProjectEngineLocationFile->GetString();
	if (FileLocation.IsEmpty())
	{
		FileLocation = FPaths::Combine(FPlatformProcess::UserSettingsDir(), *FApp::GetEpicProductIdentifier(), TEXT("Editor"), "ProjectEditorRecords.json");
	}

	return FileLocation;
}
