// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProjectManager.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Stats/Stats.h"
#include "PluginDescriptor.h"
#include "Interfaces/IPluginManager.h"
#include "ProjectDescriptor.h"
#include "Modules/ModuleManager.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/PlatformProcess.h"

DEFINE_LOG_CATEGORY_STATIC( LogProjectManager, Log, All );

#define LOCTEXT_NAMESPACE "ProjectManager"

FProjectManager::FProjectManager()
	: bIsCurrentProjectDirty(false)
{
}

const FProjectDescriptor* FProjectManager::GetCurrentProject() const
{
	return CurrentProject.Get();
}

bool FProjectManager::LoadProjectFile( const FString& InProjectFile )
{
	// Try to load the descriptor
	FText FailureReason;
	TSharedPtr<FProjectDescriptor> Descriptor = MakeShareable(new FProjectDescriptor());
	if(Descriptor->Load(InProjectFile, FailureReason))
	{
		// Create the project
		CurrentProject = Descriptor;
		CurrentProjectModuleContextInfos.Reset();
		return true;
	}

#if PLATFORM_IOS
	FString UpdatedMessage = FString::Printf(TEXT("%s\n%s"), *FailureReason.ToString(), TEXT("For troubleshooting, please go to https://docs.unrealengine.com/SharingAndReleasing/Mobile/iOS"));
	FailureReason = FText::FromString(UpdatedMessage);
#endif
	UE_LOGF(LogProjectManager, Error, "%ls", *FailureReason.ToString());
	FMessageDialog::Open(EAppMsgType::Ok, FailureReason);

	return false;
}

bool FProjectManager::LoadModulesForProject( const ELoadingPhase::Type LoadingPhase )
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Loading Game Modules"), STAT_GameModule, STATGROUP_LoadTime);

	bool bSuccess = true;

	if ( CurrentProject.IsValid() )
	{
		TMap<FName, EModuleLoadResult> ModuleLoadFailures;
		FModuleDescriptor::LoadModulesForPhase(LoadingPhase, CurrentProject->Modules, ModuleLoadFailures);

		if ( ModuleLoadFailures.Num() > 0 )
		{
			FText FailureMessage;
			for ( auto FailureIt = ModuleLoadFailures.CreateConstIterator(); FailureIt; ++FailureIt )
			{
				const EModuleLoadResult FailureReason = FailureIt.Value();

				if( FailureReason != EModuleLoadResult::Success )
				{
					const FText TextModuleName = FText::FromName(FailureIt.Key());

					if ( FailureReason == EModuleLoadResult::FileNotFound )
					{
						FailureMessage = FText::Format( LOCTEXT("PrimaryGameModuleNotFound", "The game module '{0}' could not be found. Please ensure that this module exists and that it is compiled."), TextModuleName );
					}
					else if ( FailureReason == EModuleLoadResult::FileIncompatible )
					{
						FailureMessage = FText::Format( LOCTEXT("PrimaryGameModuleIncompatible", "The game module '{0}' does not appear to be up to date. This may happen after updating the engine. Please recompile this module and try again."), TextModuleName );
					}
					else if ( FailureReason == EModuleLoadResult::FailedToInitialize )
					{
						FailureMessage = FText::Format( LOCTEXT("PrimaryGameModuleFailedToInitialize", "The game module '{0}' could not be successfully initialized after it was loaded."), TextModuleName );
					}
					else if ( FailureReason == EModuleLoadResult::CouldNotBeLoadedByOS )
					{
						FailureMessage = FText::Format( LOCTEXT("PrimaryGameModuleCouldntBeLoaded", "The game module '{0}' could not be loaded. There may be an operating system error, the module may not be properly set up, or a plugin which has been included into the build has not been turned on."), TextModuleName );
					}
					else
					{
						ensure(0);	// If this goes off, the error handling code should be updated for the new enum values!
						FailureMessage = FText::Format( LOCTEXT("PrimaryGameModuleGenericLoadFailure", "The game module '{0}' failed to load for an unspecified reason.  Please report this error."), TextModuleName );
					}

					// Just report the first error
					break;
				}
			}

			FMessageDialog::Open(EAppMsgType::Ok, FailureMessage);
			bSuccess = false;
		}
	}

	OnLoadingPhaseCompleteEvent.Broadcast(LoadingPhase, bSuccess);
	return bSuccess;
} 

bool FProjectManager::SubstituteModule(const FString& OriginalModuleName, const FString& NewModuleName)
{
	if (!CurrentProject.IsValid())
	{
		return false;
	}
	const int ModuleCount = CurrentProject->Modules.Num();
	for (int32 Idx = 0; Idx < ModuleCount; Idx++)
	{
		if (CurrentProject->Modules[Idx].Name.IsEqual(*OriginalModuleName))
		{
			CurrentProject->Modules[Idx].Name = FName(*NewModuleName);
			return true;
		}
	}
	return false;
}


#if !IS_MONOLITHIC
bool FProjectManager::CheckModuleCompatibility(TArray<FString>& OutIncompatibleModules)
{
	return !CurrentProject.IsValid() || FModuleDescriptor::CheckModuleCompatibility(CurrentProject->Modules, OutIncompatibleModules);
}
#endif

const FString& FProjectManager::GetAutoLoadProjectFileName()
{
	static FString RecentProjectFileName = FPaths::Combine(*FPaths::GameAgnosticSavedDir(), TEXT("AutoLoadProject.txt"));
	return RecentProjectFileName;
}

bool FProjectManager::SignSampleProject(const FString& FilePath, const FString& Category, FText& OutFailReason)
{
	FProjectDescriptor Descriptor;
	if(!Descriptor.Load(FilePath, OutFailReason))
	{
		return false;
	}

	Descriptor.Sign(FilePath);
	Descriptor.Category = Category;
	return Descriptor.Save(FilePath, OutFailReason);
}

bool FProjectManager::QueryStatusForProject(const FString& FilePath, FProjectStatus& OutProjectStatus) const
{
	FText FailReason;
	FProjectDescriptor Descriptor;
	if(!Descriptor.Load(FilePath, FailReason))
	{
		return false;
	}

	QueryStatusForProjectImpl(Descriptor, FilePath, OutProjectStatus);
	return true;
}

bool FProjectManager::QueryStatusForCurrentProject(FProjectStatus& OutProjectStatus) const
{
	if ( !CurrentProject.IsValid() )
	{
		return false;
	}

	QueryStatusForProjectImpl(*CurrentProject, FPaths::GetProjectFilePath(), OutProjectStatus);
	return true;
}

void FProjectManager::QueryStatusForProjectImpl(const FProjectDescriptor& ProjectInfo, const FString& FilePath, FProjectStatus& OutProjectStatus)
{
	OutProjectStatus.Name = FPaths::GetBaseFilename(FilePath);
	OutProjectStatus.Description = ProjectInfo.Description;
	OutProjectStatus.Category = ProjectInfo.Category;
	OutProjectStatus.bCodeBasedProject = ProjectInfo.Modules.Num() > 0;
	OutProjectStatus.bSignedSampleProject = ProjectInfo.IsSigned(FilePath);
	OutProjectStatus.bRequiresUpdate = ProjectInfo.FileVersion < EProjectDescriptorVersion::Latest;
	OutProjectStatus.TargetPlatforms = ProjectInfo.TargetPlatforms;
}

void FProjectManager::UpdateSupportedTargetPlatformsForProject(const FString& FilePath, const FName& InPlatformName, const bool bIsSupported)
{
	FProjectDescriptor NewProject;

	FText FailReason;
	if ( !NewProject.Load(FilePath, FailReason) )
	{
		return;
	}

	if ( bIsSupported )
	{
		NewProject.TargetPlatforms.AddUnique(InPlatformName);
	}
	else
	{
		NewProject.TargetPlatforms.Remove(InPlatformName);
	}

	NewProject.Save(FilePath, FailReason);

	// Call OnTargetPlatformsForCurrentProjectChangedEvent if this project is the same as the one we currently have loaded
	const FString CurrentProjectPath = FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath());
	const FString InProjectPath = FPaths::ConvertRelativePathToFull(FilePath);
	if ( CurrentProjectPath == InProjectPath )
	{
		OnTargetPlatformsForCurrentProjectChangedEvent.Broadcast();
	}
}

void FProjectManager::UpdateSupportedTargetPlatformsForCurrentProject(const FName& InPlatformName, const bool bIsSupported)
{
	if ( !CurrentProject.IsValid() )
	{
		return;
	}

	CurrentProject->UpdateSupportedTargetPlatforms(InPlatformName, bIsSupported);

	FText FailReason;
	CurrentProject->Save(FPaths::GetProjectFilePath(), FailReason);

	OnTargetPlatformsForCurrentProjectChangedEvent.Broadcast();
}

void FProjectManager::ClearSupportedTargetPlatformsForProject(const FString& FilePath)
{
	FProjectDescriptor Descriptor;

	FText FailReason;
	if ( !Descriptor.Load(FilePath, FailReason) )
	{
		return;
	}

	Descriptor.TargetPlatforms.Empty();
	Descriptor.Save(FilePath, FailReason);

	// Call OnTargetPlatformsForCurrentProjectChangedEvent if this project is the same as the one we currently have loaded
	const FString CurrentProjectPath = FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath());
	const FString InProjectPath = FPaths::ConvertRelativePathToFull(FilePath);
	if ( CurrentProjectPath == InProjectPath )
	{
		OnTargetPlatformsForCurrentProjectChangedEvent.Broadcast();
	}
}

void FProjectManager::ClearSupportedTargetPlatformsForCurrentProject()
{
	if ( !CurrentProject.IsValid() )
	{
		return;
	}

	CurrentProject->TargetPlatforms.Empty();

	FText FailReason;
	CurrentProject->Save(FPaths::GetProjectFilePath(), FailReason);

	OnTargetPlatformsForCurrentProjectChangedEvent.Broadcast();
}

PROJECTS_API bool HasDefaultPluginSettings(const FString& Platform)
{
	const FProjectDescriptor* CurrentProject = IProjectManager::Get().GetCurrentProject();
	bool bAllowEnginePluginsEnabledByDefault = true;

	// Get settings for the plugins which are currently enabled or disabled by the project file
	TMap<FString, bool> ConfiguredPlugins;
	if (CurrentProject != nullptr)
	{
		bAllowEnginePluginsEnabledByDefault = !CurrentProject->bDisableEnginePluginsByDefault;
		for (const FPluginReferenceDescriptor& PluginReference : CurrentProject->Plugins)
		{
			ConfiguredPlugins.Add(PluginReference.Name, PluginReference.IsEnabledForPlatform(Platform));
		}
	}

	// Check whether the setting for any default plugin has been changed
	for(const TSharedRef<IPlugin>& Plugin: IPluginManager::Get().GetDiscoveredPlugins())
	{
		if (Plugin->GetDescriptor().SupportsTargetPlatform(Platform))
		{
			const bool bEnabledByDefault = Plugin->IsEnabledByDefault(bAllowEnginePluginsEnabledByDefault);
			bool bEnabled = bEnabledByDefault;

			bool* EnabledPtr = ConfiguredPlugins.Find(Plugin->GetName());
			if(EnabledPtr != nullptr)
			{
				bEnabled = *EnabledPtr;
			}

			bool bEnabledInDefaultExe = (Plugin->GetLoadedFrom() == EPluginLoadedFrom::Engine && bEnabledByDefault && !Plugin->GetDescriptor().bInstalled);
			if(bEnabled != bEnabledInDefaultExe)
			{
				return false;
			}
		}
	}

	return true;
}

bool FProjectManager::HasDefaultPluginSettings() const
{
	return ::HasDefaultPluginSettings(FPlatformMisc::GetUBTPlatform());
}

bool FProjectManager::SetPluginEnabled(const FString& PluginName, bool bEnabled, FText& OutFailReason)
{
	// Don't go any further if there's no project loaded
	if (!CurrentProject.IsValid())
	{
		OutFailReason = LOCTEXT("NoProjectLoaded", "No project is currently loaded");
		return false;
	}

	bool bProjectChanged = false;

	// Find or create the index of any existing reference in the project descriptor	
	int32 PluginRefIdx = 0;
	for (;; PluginRefIdx++)
	{
		if (PluginRefIdx == CurrentProject->Plugins.Num())
		{
			PluginRefIdx = CurrentProject->Plugins.Add(FPluginReferenceDescriptor(PluginName, bEnabled));
			bProjectChanged = true;
			break;
		}
		else if (CurrentProject->Plugins[PluginRefIdx].Name == PluginName)
		{
			if (CurrentProject->Plugins[PluginRefIdx].bEnabled != bEnabled)
			{
				CurrentProject->Plugins[PluginRefIdx].bEnabled = bEnabled;
				bProjectChanged = true;
			}
			break;
		}
	}

	// Remove any other references to the plugin
	for (int Idx = CurrentProject->Plugins.Num() - 1; Idx > PluginRefIdx; Idx--)
	{
		if (CurrentProject->Plugins[Idx].Name == PluginName)
		{
			CurrentProject->Plugins.RemoveAt(Idx);
			bProjectChanged = true;
		}
	}

	// Get a reference to the plugin reference we need to update
	FPluginReferenceDescriptor& PluginRef = CurrentProject->Plugins[PluginRefIdx];

	// Update the plugin reference with metadata from the plugin instance
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
	if (Plugin.IsValid())
	{
		const FPluginDescriptor& PluginDescriptor = Plugin->GetDescriptor();
		if (PluginRef.MarketplaceURL != PluginDescriptor.MarketplaceURL)
		{
			PluginRef.MarketplaceURL = PluginDescriptor.MarketplaceURL;
			bProjectChanged = true;
		}

		if (PluginRef.SupportedTargetPlatforms != PluginDescriptor.SupportedTargetPlatforms)
		{
			PluginRef.SupportedTargetPlatforms = PluginDescriptor.SupportedTargetPlatforms;
			bProjectChanged = true;
		}
	}

	// If the current plugin reference is the default, just remove it from the list
	if (PluginRef.PlatformAllowList.Num() == 0 && PluginRef.PlatformDenyList.Num() == 0)
	{
		// We alway need to be explicit about installed plugins, because they'll be auto-enabled again if we're not.
		if (!Plugin.IsValid() || !Plugin->GetDescriptor().bInstalled)
		{
			// Get the default list of enabled plugins
			TArray<FString> DefaultEnabledPlugins;
			GetDefaultEnabledPlugins(DefaultEnabledPlugins, false, !CurrentProject->bDisableEnginePluginsByDefault);

			// Check the enabled state is the same in that
			if (DefaultEnabledPlugins.Contains(PluginName) == bEnabled)
			{
				CurrentProject->Plugins.RemoveAt(PluginRefIdx);
				bProjectChanged = true;
			}
		}
	}

	if (bProjectChanged)
	{
		// Mark project as dirty
		bIsCurrentProjectDirty = true;
	}

	return true;
}

bool FProjectManager::RemovePluginReference(const FString& PluginName, FText& OutFailReason)
{
	// Don't go any further if there's no project loaded
	if (!CurrentProject.IsValid())
	{
		OutFailReason = LOCTEXT("NoProjectLoaded", "No project is currently loaded");
		return false;
	}

	bool bPluginFound = false;
	for (int32 PluginRefIdx = CurrentProject->Plugins.Num() - 1; PluginRefIdx >= 0 && !bPluginFound; --PluginRefIdx)
	{
		if (CurrentProject->Plugins[PluginRefIdx].Name == PluginName)
		{
			CurrentProject->Plugins.RemoveAt(PluginRefIdx);
			bPluginFound = true;
			break;
		}
	}
	return bPluginFound;
}

void FProjectManager::GetDefaultEnabledPlugins(TArray<FString>& OutPluginNames, bool bIncludeInstalledPlugins, bool bAllowEnginePluginsEnabledByDefault)
{
	// Add all the game plugins and everything marked as enabled by default
	for (TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetDiscoveredPlugins())
	{
		if(Plugin->IsEnabledByDefault(bAllowEnginePluginsEnabledByDefault))
		{
			if(bIncludeInstalledPlugins || !Plugin->GetDescriptor().bInstalled)
			{
				OutPluginNames.AddUnique(Plugin->GetName());
			}
		}
	}
}

bool FProjectManager::UpdateAdditionalPluginDirectory(const FString& InDir, const bool bAddOrRemove)
{
	if (!CurrentProject.IsValid())
	{
		return false;
	}

	const FString FullPathNormalizedDir = FPaths::ConvertRelativePathToFull(InDir);
	bool bChanged = false;
	if (bAddOrRemove)
	{
		bChanged = CurrentProject->AddPluginDirectory(FullPathNormalizedDir);
	}
	else
	{
		bChanged = CurrentProject->RemovePluginDirectory(FullPathNormalizedDir);
	}

	if (bChanged)
	{
		FText FailReason;
		SaveCurrentProjectToDisk(FailReason);
	}

	return bChanged;
}

const TArray<FString>& FProjectManager::GetAdditionalPluginDirectories() const
{
	if (!CurrentProject.IsValid())
	{
		static const TArray<FString> EmptyList;
		return EmptyList;
	}
	return CurrentProject->GetAdditionalPluginDirectories();
}

bool FProjectManager::IsCurrentProjectDirty() const
{
	return bIsCurrentProjectDirty;
}

bool FProjectManager::SaveCurrentProjectToDisk(FText& OutFailReason)
{
	if (CurrentProject->Save(FPaths::GetProjectFilePath(), OutFailReason))
	{
		bIsCurrentProjectDirty = false;
		return true;
	}
	return false;
}

bool FProjectManager::IsEnterpriseProject()
{
	bool bIsEnterprise = false;
	if (CurrentProject.IsValid())
	{
		bIsEnterprise = CurrentProject->bIsEnterpriseProject;
	}

	return bIsEnterprise;
}

void FProjectManager::SetIsEnterpriseProject(bool bValue)
{
	if (CurrentProject.IsValid())
	{
		CurrentProject->bIsEnterpriseProject = bValue;
	}
}

TArray<FModuleContextInfo>& FProjectManager::GetCurrentProjectModuleContextInfos()
{
	return CurrentProjectModuleContextInfos;
}

bool FProjectManager::IsSuppressingProjectFileWrite() const
{
	return SuppressProjectFileWriteList.Num() > 0;
}

void FProjectManager::AddSuppressProjectFileWrite(const FName InName)
{
	SuppressProjectFileWriteList.AddUnique(InName);
}

void FProjectManager::RemoveSuppressProjectFileWrite(const FName InName)
{
	SuppressProjectFileWriteList.Remove(InName);
}

IProjectManager& IProjectManager::Get()
{
	// Single instance of manager, allocated on demand and destroyed on program exit.
	static FProjectManager* ProjectManager = NULL;
	if( ProjectManager == NULL )
	{
		SCOPED_BOOT_TIMING("new FProjectManager");
		ProjectManager = new FProjectManager();
	}
	return *ProjectManager;
}

#undef LOCTEXT_NAMESPACE
