// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IProjectManager.h"

struct FProjectDescriptor;

/**
 * ProjectAndPluginManager manages available code and content extensions (both loaded and not loaded.)
 */
class FProjectManager final : public IProjectManager
{
public:
	FProjectManager();

	/** IProjectManager interface */
	virtual const FProjectDescriptor *GetCurrentProject() const override;
	virtual bool LoadProjectFile( const FString& ProjectFile ) override;
	virtual bool LoadModulesForProject( const ELoadingPhase::Type LoadingPhase ) override;
	virtual FLoadingModulesForPhaseEvent& OnLoadingPhaseComplete() override { return OnLoadingPhaseCompleteEvent; }
	virtual bool SubstituteModule(const FString& OriginalModuleName, const FString& NewModuleName) override;
#if !IS_MONOLITHIC
	virtual bool CheckModuleCompatibility( TArray<FString>& OutIncompatibleModules ) override;
#endif
	virtual const FString& GetAutoLoadProjectFileName() override;
	virtual bool SignSampleProject(const FString& FilePath, const FString& Category, FText& OutFailReason) override;
	virtual bool QueryStatusForProject(const FString& FilePath, FProjectStatus& OutProjectStatus) const override;
	virtual bool QueryStatusForCurrentProject(FProjectStatus& OutProjectStatus) const override;
	virtual void UpdateSupportedTargetPlatformsForProject(const FString& FilePath, const FName& InPlatformName, const bool bIsSupported) override;
	virtual void UpdateSupportedTargetPlatformsForCurrentProject(const FName& InPlatformName, const bool bIsSupported) override;
	virtual void ClearSupportedTargetPlatformsForProject(const FString& FilePath) override;
	virtual void ClearSupportedTargetPlatformsForCurrentProject() override;
	virtual FOnTargetPlatformsForCurrentProjectChangedEvent& OnTargetPlatformsForCurrentProjectChanged() override { return OnTargetPlatformsForCurrentProjectChangedEvent; }
	virtual bool HasDefaultPluginSettings() const override;
	virtual bool SetPluginEnabled(const FString& PluginName, bool bEnabled, FText& OutFailReason) override;
	virtual bool RemovePluginReference(const FString& PluginName, FText& OutFailReason) override;
	virtual bool UpdateAdditionalPluginDirectory(const FString& Dir, const bool bAddOrRemove) override;
	virtual const TArray<FString>& GetAdditionalPluginDirectories() const override;
	virtual bool IsCurrentProjectDirty() const override;
	virtual bool SaveCurrentProjectToDisk(FText& OutFailReason) override;
	virtual bool IsEnterpriseProject() override;
	virtual void SetIsEnterpriseProject(bool bValue) override;
	virtual TArray<FModuleContextInfo>& GetCurrentProjectModuleContextInfos() override;
	virtual bool IsSuppressingProjectFileWrite() const override;
	virtual void AddSuppressProjectFileWrite(const FName InName) override;
	virtual void RemoveSuppressProjectFileWrite(const FName InName) override;

private:
	static void QueryStatusForProjectImpl(const FProjectDescriptor& Project, const FString& FilePath, FProjectStatus& OutProjectStatus);

	/** Gets the list of plugins enabled by default, excluding the project overrides */
	static void GetDefaultEnabledPlugins(TArray<FString>& PluginNames, bool bIncludeInstalledPlugins, bool bAllowEnginePluginsEnabledByDefault);

	/** The project that is currently loaded in the editor */
	TSharedPtr< FProjectDescriptor > CurrentProject;

	/** Cached list of module infos for the project that is currently loaded in the editor */
	TArray<FModuleContextInfo> CurrentProjectModuleContextInfos;

	/** Whether the current project has been modified but not saved to disk */
	bool bIsCurrentProjectDirty;

	/** Delegate called when the target platforms for the current project are changed */
	FOnTargetPlatformsForCurrentProjectChangedEvent OnTargetPlatformsForCurrentProjectChangedEvent;

	/** Delegate called when LoadModulesForProject() completes for a particular phase */
	FLoadingModulesForPhaseEvent OnLoadingPhaseCompleteEvent;

	/** Array of names that have disabled project file writes */
	TArray<FName> SuppressProjectFileWriteList;
};


