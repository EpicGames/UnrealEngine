// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProjectUtilities.h"
#include "Containers/UnrealString.h"
#include "HAL/FileManagerGeneric.h"
#include "HAL/PlatformMisc.h"
#include "Misc/Paths.h"
#include "ProjectDescriptor.h"

namespace UE::ProjectUtilities
{

void ParseProjectDirFromCommandline(int32 ArgC, TCHAR* ArgV[])
{
	// Although standalone tools can set the project path via the cmdline this will not change the ProjectDir being
	// used as standalone tools have a bespoke path in FGenericPlatformMisc::ProjectDir. We can address this
	// by doing our own parsing then using the project dir override feature.
	if (ArgC >= 2)
	{
		FString Cmd(ArgV[1]);

		if (!Cmd.IsEmpty() && !Cmd.StartsWith(TEXT("-")) && Cmd.EndsWith(FProjectDescriptor::GetExtension()))
		{
			FString ProjectDir = FPaths::GetPath(Cmd);
			ProjectDir = FFileManagerGeneric::DefaultConvertToRelativePath(*ProjectDir);

			// The path should end with a trailing slash (see FGenericPlatformMisc::ProjectDir) so we use
			// NormalizeFilename not NormalizeDirectoryName as the latter will remove trailing slashes. We
			// also need to add one if it is missing.
			FPaths::NormalizeFilename(ProjectDir);
			if (!ProjectDir.EndsWith(TEXT("/")))
			{
				ProjectDir += TEXT("/");
			}

			FPlatformMisc::SetOverrideProjectDir(ProjectDir);
		}
	}
}

}
