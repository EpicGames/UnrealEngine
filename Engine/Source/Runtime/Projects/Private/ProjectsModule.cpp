// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ProjectsModule.cpp: Implements the FProjectsModule class.
=============================================================================*/

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


/**
 * Implements the Projects module.
 */
class FProjectsModule
	: public IModuleInterface
{
public:

	virtual void StartupModule( ) override { }

	virtual void ShutdownModule( ) override { }

	virtual bool SupportsDynamicReloading( ) override
	{
		return true;
	}
};


IMPLEMENT_MODULE(FProjectsModule, Projects);
