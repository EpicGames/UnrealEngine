// Copyright Epic Games, Inc. All Rights Reserved.

#include "RapidJsonPluginLoading.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "RapidJsonPluginLoading"

DEFINE_LOG_CATEGORY_STATIC(LogRapidJsonPluginLoading, Display, All);

namespace UE {
namespace Projects {
namespace Private {

// the DOM json parser is case insensitive on keys names
Json::FValue::ConstMemberIterator FindMemberCaseInsensitive(Json::FConstObject Object, const TCHAR* FieldName)
{
	const int32 FieldNameLength = FCString::Strlen(FieldName);

	for (Json::FValue::ConstMemberIterator MemberIt = Object.MemberBegin(); MemberIt != Object.MemberEnd(); ++MemberIt)
	{
		if (FieldNameLength != MemberIt->name.GetStringLength())
		{
			continue;
		}

		if (FCString::Strnicmp(FieldName, MemberIt->name.GetString(), FieldNameLength) == 0)
		{
			return MemberIt;
		}
	}

	return Object.MemberEnd();
}

bool TryGetBoolField(Json::FConstObject Object, const TCHAR* FieldName, bool& Out)
{
	Json::FValue::ConstMemberIterator It = FindMemberCaseInsensitive(Object, FieldName);
	if (It == Object.MemberEnd())
	{
		return false;
	}

	if (It->value.IsBool())
	{
		Out = It->value.GetBool();
		return true;
	}
	else if (It->value.IsInt64())
	{
		Out = (It->value.GetInt64() != 0);
		return true;
	}
	else if (It->value.IsUint64())
	{
		Out = (It->value.GetUint64() != 0);
		return true;
	}
	else if (It->value.IsDouble())
	{
		Out = (It->value.GetDouble() != 0.0);
		return true;
	}
	else if (It->value.IsString())
	{
		Out = TCString<TCHAR>::ToBool(It->value.GetString());
		return true;
	}

	return false;
}

// this 'TryConvertFromDouble' code was copy/pasted from the FJsonValue code, to maintain compatibility. When the data converted from the shared json for a 
// legacy code path it loses any of the original type meaning. So parsing from this double version needs to behave the same as if we were reading form integers
//
// As soon as the public FJsonObject read API can be deprecated this can be removed
template <typename T>
bool TryConvertFromDouble(double Double, T& OutNumber)
{
	if ((Double >= TNumericLimits<T>::Min()) && (Double <= static_cast<double>(TNumericLimits<T>::Max())))
	{
		OutNumber = static_cast<T>(FMath::RoundHalfFromZero(Double));

		return true;
	}

	return false;
}

// Need special handling for int64/uint64, due to overflow in the numeric limits.
// 2^63-1 and 2^64-1 cannot be exactly represented as a double, so TNumericLimits<>::Max() gets rounded up to exactly 2^63 or 2^64 by the compiler's implicit cast to double.
// This breaks the overflow check in TryConvertNumber. We use "<" rather than "<=" along with the exact power-of-two double literal to fix this.
template <> 
bool TryConvertFromDouble<uint64>(double Double, uint64& OutNumber)
{
	if (Double >= 0.0 && Double < 18446744073709551616.0)
	{
		OutNumber = static_cast<uint64>(FMath::RoundHalfFromZero(Double));
		return true;
	}

	return false;
}

template <>
bool TryConvertFromDouble<int64>(double Double, int64& OutNumber)
{
	if (Double >= -9223372036854775808.0 && Double < 9223372036854775808.0)
	{
		OutNumber = static_cast<int64>(FMath::RoundHalfFromZero(Double));
		return true;
	}

	return false;
}

// returns true if the value can be safely stored in output
template<typename T>
static inline bool TryGetIntHelper(Json::FConstObject Object, const TCHAR* FieldName, const TCHAR* OutTypeName, T& Out)
{
	using namespace Json;

	FValue::ConstMemberIterator It = FindMemberCaseInsensitive(Object, FieldName);
	if (It == Object.MemberEnd())
	{
		return false;
	}

	if (It->value.IsInt64())
	{
		const int64 Value = It->value.GetInt64();
		if (IntFitsIn<T>(Value))
		{
			Out = static_cast<T>(Value);
			return true;
		}
		else
		{
			return false;
		}
	}
	else if (It->value.IsUint64())
	{
		const uint64 Value = It->value.GetUint64();
		if (IntFitsIn<T>(Value))
		{
			Out = static_cast<T>(Value);
			return true;
		}
		else
		{
			return false;
		}
	}
	else if (It->value.IsDouble())
	{
		const double Value = It->value.GetDouble();
		if (TryConvertFromDouble<T>(Value, Out))
		{
			return true;
		}
		else
		{
			return false;
		}
	}
	else if (It->value.IsBool())
	{
		Out = It->value.GetBool() ? 1 : 0;
		return true;
	}
	else if (It->value.IsString())
	{
		LexFromString(Out, It->value.GetString());
		return true;
	}
	else
	{
		return false;
	}
}

bool TryGetNumberField(Json::FConstObject Object, const TCHAR* FieldName, int32& Out)
{
	return TryGetIntHelper(Object, FieldName, TEXT("int32"), Out);
}

bool TryGetNumberField(Json::FConstObject Object, const TCHAR* FieldName, uint32& Out)
{
	return TryGetIntHelper(Object, FieldName, TEXT("uint32"), Out);
}

static inline bool ConvertToStringHelper(const Json::FValue& Value, FString& Out)
{
	if (Value.IsString())
	{
		Out = Value.GetString();
		return true;
	}
	else if (Value.IsInt64())
	{
		Out = FString::SanitizeFloat(static_cast<double>(Value.GetInt64()), 0);
		return true;
	}
	else if (Value.IsDouble())
	{
		Out = FString::SanitizeFloat(Value.GetDouble(), 0);
		return true;
	}
	else if (Value.IsBool())
	{
		Out = Value.GetBool() ? TEXT("true") : TEXT("false");
		return true;
	}
	else
	{
		Out.Reset();
		return false;
	}
}

bool TryGetStringField(Json::FConstObject Object, const TCHAR* FieldName, FString& Out)
{
	using namespace Json;

	FValue::ConstMemberIterator It = FindMemberCaseInsensitive(Object, FieldName);
	if (It == Object.MemberEnd())
	{
		return false;
	}

	if (ConvertToStringHelper(It->value, Out))
	{
		return true;
	}

	return false;
}

bool TryGetStringArrayField(Json::FConstObject Object, const TCHAR* FieldName, TArray<FString>& Out)
{
	using namespace Json;

	Out.Reset();

	FValue::ConstMemberIterator It = FindMemberCaseInsensitive(Object, FieldName);
	if (It == Object.MemberEnd() || !It->value.IsArray())
	{
		return false;
	}

	FConstArray Array = It->value.GetArray();
	Out.Reserve(Array.Size());
	for (const FValue& Item : Array)
	{
		if (!ConvertToStringHelper(Item, Out.Emplace_GetRef()))
		{
			Out.Pop();
			return false;
		}
	}

	return true;
}

bool TryGetStringArrayField(Json::FConstObject Object, const TCHAR* FieldName, TArray<FName>& Out)
{
	using namespace Json;

	Out.Reset();

	FValue::ConstMemberIterator It = FindMemberCaseInsensitive(Object, FieldName);
	if (It == Object.MemberEnd() || !It->value.IsArray())
	{
		return false;
	}

	FConstArray Array = It->value.GetArray();
	Out.Reserve(Array.Size());

	FString TempString;

	for (const FValue& Item : Array)
	{
		if (ConvertToStringHelper(Item, TempString))
		{
			Out.Emplace(TempString);
		}
		else
		{
			return false;
		}
	}

	return true;
}

bool TryGetStringArrayFieldWithDeprecatedFallback(Json::FConstObject Object, const TCHAR* FieldName, const TCHAR* DeprecatedFieldName, TArray<FString>& OutArray)
{
	if (TryGetStringArrayField(Object, FieldName, OutArray))
	{
		return true;
	}

	return TryGetStringArrayField(Object, DeprecatedFieldName, OutArray);
}	

FText GetArrayObjectTypeError(const TCHAR* FieldName, int32 Index)
{
	return FText::Format(LOCTEXT("InvalidPluginArrayEntry", "Field '{0}[{1}]' is a non-object entry"), FText::FromString(FieldName), Index);
}


FText GetArrayObjectChildParseError(const TCHAR* FieldName, int32 Index, const FText& PropagateError)
{
	return FText::Format(LOCTEXT("InvalidPluginParseArrayEntry", "'{0}[{1}]': {2}"), FText::FromString(FieldName), Index, PropagateError);
}

} // Private
} // Projects
} // UE

#undef LOCTEXT_NAMESPACE