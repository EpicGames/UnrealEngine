// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "JsonUtils/RapidJsonUtils.h"
#include "JsonUtils/JsonConversion.h"
#include "Internationalization/Text.h"

struct FLocalizationTargetDescriptor;
struct FModuleDescriptor;
struct FPluginDescriptor;
struct FCustomBuildSteps;
struct FPluginReferenceDescriptor;
struct FPluginDisallowedDescriptor;
struct FPluginManifest;

namespace UE {
namespace Projects {
namespace Private {

/**
 * These are utility functions to be as close to the default Json parser as possible, only really necessary
 * because we have public APIs that take FJsonObjects directly
*/

bool TryGetBoolField(Json::FConstObject Object, const TCHAR* FieldName, bool& Out);
bool TryGetNumberField(Json::FConstObject Object, const TCHAR* FieldName, int32& Out);
bool TryGetNumberField(Json::FConstObject Object, const TCHAR* FieldName, uint32& Out);
bool TryGetStringField(Json::FConstObject Object, const TCHAR* FieldName, FString& Out);
bool TryGetStringField(Json::FConstObject Object, const TCHAR* FieldName, FStringView& Out);
bool TryGetStringArrayField(Json::FConstObject Object, const TCHAR* FieldName, TArray<FString>& Out);
bool TryGetStringArrayField(Json::FConstObject Object, const TCHAR* FieldName, TArray<FName>& Out);	

bool TryGetStringArrayFieldWithDeprecatedFallback(Json::FConstObject Object, const TCHAR* FieldName, const TCHAR* DeprecatedFieldName, TArray<FString>& OutArray);

FText GetArrayObjectTypeError(const TCHAR* FieldName, int32 Index);
FText GetArrayObjectChildParseError(const TCHAR* FieldName, int32 Index, const FText& PropagateError);


/** Get the field named FieldName as an array of strings. Returns false if it doesn't exist or any member cannot be converted. */
template<typename TEnum>
inline bool TryGetEnumArrayField(Json::FConstObject Object, const TCHAR* FieldName, TArray<TEnum>& OutArray)
{
	TOptional<Json::FConstArray> Array = Json::GetArrayField(Object, FieldName);
	if (!Array.IsSet())
	{
		return false;
	}

	OutArray.Reset(Array->Size());
	for (const Json::FValue& JsonValue : *Array)
	{
		if (!JsonValue.IsString())
		{
			continue;
		}

		TEnum Value;
		if (LexTryParseString(Value, JsonValue.GetString()))
		{
			OutArray.Add(Value);
		}
	}
	return true;
}

template<typename TEnum>
inline bool TryGetEnumArrayFieldWithDeprecatedFallback(Json::FConstObject Object, const TCHAR* FieldName, const TCHAR* DeprecatedFieldName, TArray<TEnum>& OutArray)
{
	if (TryGetEnumArrayField<TEnum>(Object, FieldName, /*out*/ OutArray))
	{
		return true;
	}
	else if (TryGetEnumArrayField<TEnum>(Object, DeprecatedFieldName, /*out*/ OutArray))
	{
		//@TODO: Warn about deprecated field fallback?
		return true;
	}
	else
	{
		return false;
	}
}

// A private interface as to not expose RapidJson types

TOptional<FText> Read(Json::FConstObject Object, FPluginManifest& Out);
TOptional<FText> Read(Json::FConstObject Object, FPluginDescriptor& Out);
TOptional<FText> Read(Json::FConstObject Object, FModuleDescriptor& Out);
TOptional<FText> Read(Json::FConstObject Object, FLocalizationTargetDescriptor& Out);
TOptional<FText> Read(Json::FConstObject Object, FPluginReferenceDescriptor& Out);
TOptional<FText> Read(Json::FConstObject Object, FPluginDisallowedDescriptor& Out);

FCustomBuildSteps ReadCustomBuildSteps(Json::FConstObject Object, const TCHAR* FieldName);

// Helper functions to convert and call the rapidjson functions from default json sources
template<typename FUNCTOR>
bool ReadFromDefaultJsonHelper(const FJsonObject& Object, FUNCTOR && Functor)
{
	TOptional<Json::FDocument> Document = Json::ConvertSharedJsonToRapidJsonDocument(Object);
	if (!Document.IsSet())
	{
		return false;
	}

	TOptional<Json::FConstObject> RootObject = Json::GetRootObject(*Document);
	if (!RootObject.IsSet())
	{
		return false;
	}

	return Functor(*RootObject);
}

template<typename TYPE>
bool ReadFromDefaultJson(const FJsonObject& Object, TYPE& Out, FText* OutFailReason)
{
	return ReadFromDefaultJsonHelper(
		Object,
		[&Out, OutFailReason](Json::FConstObject RootObject)
		{
			TOptional<FText> ResultError = Read(RootObject, Out);
			if (ResultError)
			{
				if (OutFailReason)
				{
					*OutFailReason = *ResultError;
				}

				return false;
			}

			return true;
		}
	);
}	

template<typename TYPE>
TOptional<FText> ReadArray(Json::FConstObject Object, const TCHAR* FieldName, TArray<TYPE>& OutArray)
{
	TOptional<Json::FConstArray> ArrayValue = Json::GetArrayField(Object, FieldName);
	if(ArrayValue.IsSet())
	{
		OutArray.Reserve(ArrayValue->Size());

		for (const Json::FValue& ArrayItem : *ArrayValue)
		{
			if (ArrayItem.IsObject())
			{
				TOptional<FText> EntryResult = Read(ArrayItem.GetObject(), OutArray.Emplace_GetRef());
				if (EntryResult.IsSet())
				{
					OutArray.Pop();
					return GetArrayObjectChildParseError(FieldName, OutArray.Num(), *EntryResult);
				}
			}
			else
			{
				return GetArrayObjectTypeError(FieldName, OutArray.Num());
			}
		}
	}
	
	return {};
}

template<typename TYPE>
bool ReadArrayFromDefaultJson(const FJsonObject& Object, const TCHAR* Name, TArray<TYPE>& OutArray, FText* OutFailReason)
{
	return ReadFromDefaultJsonHelper(
		Object,
		[Name, &OutArray, OutFailReason](Json::FConstObject RootObject)
		{
			TOptional<FText> ArrayError = ReadArray<TYPE>(RootObject, Name, OutArray);
			if (ArrayError)
			{
				if (OutFailReason)
				{
					*OutFailReason = MoveTemp(*ArrayError);
				}

				return false;
			}

			return true;
		}
	);
}	
	
}
}  
}