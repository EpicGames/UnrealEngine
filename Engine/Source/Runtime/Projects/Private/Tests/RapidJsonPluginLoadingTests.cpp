// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Serialization/JsonSerializer.h"
#include "Tests/TestHarnessAdapter.h"
#include "RapidJsonPluginLoading.h"
#include "PluginDescriptor.h"
#include "ProjectDescriptor.h"

#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

/** Ensure RapidJsonUtils parses with the same quirks as the default Json loader, e.g. converting between types correctly **/
TEST_CASE_NAMED(FProjectsLoadingRapidJsonPluginLoading, "System::Engine::Projects::Loading::RapidJsonPluginLoading", "[ApplicationContextMask][SmokeFilter]")
{
	const TCHAR JsonText[] = TEXT(R"json(
{
	"zero": 0,
	"min_int32": -2147483648,
	"max_int32":  2147483647,
	"max_uint32": 4294967295,
	"round_down": 1.1,
	"round_up" : 1.9,
	"negative_round_down": -1.9,
	"negative_round_up" : -1.1,
	"string": "hello world",
	"on": true,
	"off": false,
	"object": {
		"value": 42
	},
	"array": [
		"aaa",
		"bbb",
		"ccc",
		"ddd"
	],	
	"null": null
}

)json");

	// check to make sure all values are parsed the same across RapidJsonUtils an default Json

	TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(JsonText);
	TSharedPtr< FJsonObject > DefaultRoot;
	REQUIRE(FJsonSerializer::Deserialize(Reader, DefaultRoot));

	using namespace UE::Json;
	using namespace UE::Projects::Private;

	// convert to RapidJson
	TValueOrError<FDocument, FParseError> RapidJsonDocument = UE::Json::Parse(JsonText);
	FString ParseErrorMessage = RapidJsonDocument.HasError() ? RapidJsonDocument.GetError().CreateMessage(JsonText) : TEXT("No Error");
	REQUIRE_MESSAGE(ParseErrorMessage, RapidJsonDocument.HasValue());

	TOptional<UE::Json::FConstObject> RapidRoot = UE::Json::GetRootObject(RapidJsonDocument.GetValue());
	REQUIRE(RapidRoot.IsSet());

	auto CheckField = [&Default = DefaultRoot, Rapid = *RapidRoot](const TCHAR* FieldName)
	{
		int32 DefaultInt32{};
		int32 RapidInt32{};
		const bool bDefaultInt32Found = Default->TryGetNumberField(FieldName, DefaultInt32);
		const bool bRapidInt32Found = TryGetNumberField(Rapid, FieldName, RapidInt32);
		REQUIRE_MESSAGE(*FString::Printf(TEXT("%s: int32 Parse Mismatch: %d vs %d"), FieldName, int32(bDefaultInt32Found), int32(bRapidInt32Found)), bDefaultInt32Found == bRapidInt32Found);
		REQUIRE_MESSAGE(*FString::Printf(TEXT("%s: int32 Mismatch: %d vs %d"), FieldName, DefaultInt32, RapidInt32), DefaultInt32 == RapidInt32);

		uint32 DefaultUint32{};
		uint32 RapidUint32{};
		const bool bDefaultUint32Found = Default->TryGetNumberField(FieldName, DefaultUint32);
		const bool bRapidUint32Found = TryGetNumberField(Rapid, FieldName, RapidUint32);
		REQUIRE_MESSAGE(*FString::Printf(TEXT("%s: uint32 Parse Mismatch: %d vs %d"), FieldName, int32(bDefaultUint32Found), int32(bRapidUint32Found)), bDefaultUint32Found == bRapidUint32Found);
		REQUIRE_MESSAGE(*FString::Printf(TEXT("%s: uint32 Mismatch: %u vs %u"), FieldName, DefaultUint32, RapidUint32), DefaultUint32 == RapidUint32);

		bool DefaultBool{};
		bool RapidBool{};
		const bool bDefaultBoolFound = Default->TryGetBoolField(FieldName, DefaultBool);
		const bool bRapidBoolFound = TryGetBoolField(Rapid, FieldName, RapidBool);
		REQUIRE_MESSAGE(*FString::Printf(TEXT("%s: Bool Parse Mismatch: %d vs %d"), FieldName, int32(bDefaultBoolFound), int32(bRapidBoolFound)), bDefaultBoolFound == bRapidBoolFound);
		REQUIRE_MESSAGE(TEXT("Bool Mismatch"), DefaultBool == RapidBool);

		FString DefaultString{};
		FString RapidString{};
		const bool bDefaultStringFound = Default->TryGetStringField(FieldName, DefaultString);
		const bool bRapidStringFound = TryGetStringField(Rapid, FieldName, RapidString);
		REQUIRE_MESSAGE(*FString::Printf(TEXT("%s: String Parse Mismatch: %d vs %d"), FieldName, int32(bDefaultStringFound), int32(bRapidStringFound)), bDefaultStringFound == bRapidStringFound);
		REQUIRE_MESSAGE(TEXT("String Mismatch"), DefaultString == RapidString);

		const TArray<TSharedPtr<FJsonValue>>* DefaultArray{};
		TOptional<FConstArray> RapidArray = GetArrayField(Rapid, FieldName);
		REQUIRE_MESSAGE(*FString::Printf(TEXT("%s: Array Parse Mismatch"), FieldName), Default->TryGetArrayField(FieldName, DefaultArray) == RapidArray.IsSet());

		if (DefaultArray && RapidArray.IsSet())
		{
			REQUIRE_MESSAGE(TEXT("Array Size Mismatch"), DefaultArray->Num() == RapidArray->Size());
		}

		const TSharedPtr<FJsonObject>* DefaultChildObject{};
		TOptional<FConstObject> RapidChildObject = GetObjectField(Rapid, FieldName);
		REQUIRE_MESSAGE(*FString::Printf(TEXT("%s: Object Parse Mismatch"), FieldName), Default->TryGetObjectField(FieldName, DefaultChildObject) == RapidChildObject.IsSet());

		TSharedPtr<FJsonValue> DefaultNull = Default->TryGetField(FieldName);
		const bool HasDefaultNull = DefaultNull ? DefaultNull->Type == EJson::Null : false;
		const bool HasRapidNull = HasNullField(Rapid, FieldName);

		REQUIRE_MESSAGE(*FString::Printf(TEXT("%s: Null Type Parse Mismatch"), FieldName), HasDefaultNull == HasRapidNull);
	};

	CheckField(TEXT("zero"));
	CheckField(TEXT("min_int32"));
	CheckField(TEXT("max_int32"));
	CheckField(TEXT("max_uint32"));
	CheckField(TEXT("round_down"));
	CheckField(TEXT("round_up"));
	CheckField(TEXT("negative_round_down"));
	CheckField(TEXT("negative_round_up"));
	CheckField(TEXT("string"));
	CheckField(TEXT("on"));
	CheckField(TEXT("off"));
	CheckField(TEXT("object"));
	CheckField(TEXT("array"));
	CheckField(TEXT("null"));
}


// the name of the test case seems scrambled sometimes, so make a consistent helper struct that we can friend to test this function
struct FProjectsLoadingPluginDescriptorTestsHelper
{
	static bool Read(FProjectDescriptor& ProjectDescriptor, const UE::Json::FConstObject Object, const FString& PathToProject, FText& OutFailReason)
	{
		return ProjectDescriptor.Read(Object, PathToProject, OutFailReason);
	}
};

// make sure everything in the plugin descriptor loads
TEST_CASE_NAMED(FProjectsLoadingPluginDescriptorTests, "System::Engine::Projects::Loading::PluginDescriptor", "[ApplicationContextMask][SmokeFilter]")
{
	// every property is here and set to a non-default value to ensure it reads
	const TCHAR JsonText[] = TEXT(R"json(
{
	"FileVersion": 3,
	"Version": 8,
	"VersionName": "0.2",
	"FriendlyName": "Test Plugin",
	"Description": "Test Plugin for unit testing",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "DocsURL goes here",
	"MarketplaceURL": "MarketplaceURL goes here",
	"SupportURL": "SupportURL goes here",
	"EngineVersion": "5.5",
	"EditorCustomVirtualPath": "EditorCustomVirtualPath goes here",
	"SupportedTargetPlatforms": ["Win64", "Mac"],
	"SupportedPrograms": ["UnrealInsights"],
	"bIsPluginExtension": true,
	"EnabledByDefault": false,
	"Installed": true,
	"VersePath": "/VersePath.com",
	"DeprecatedEngineVersion": "5.5",
	"VerseScope": "InternalAPI",
	"VerseVersion": 1,
	"EnableSceneGraph": true,
	"EnableSceneGraphAnimation": true,
	"EnableVerseAssetReflection": true,
	"EnableIAD": true,
	"CanContainContent": true,
	"CanContainVerse": true,
	"NoCode": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": true,
	"RequiresBuildPlatform": true,
	"Hidden": true,
	"Sealed": true,
	"ExplicitlyLoaded": true,
	"HasExplicitPlatforms": true,
	"CanBeUsedWithUnrealHeaderTool": true,
	"LocalizationTargets": [
		{
			"Name": "LocalizationTarget AAA",
			"LoadingPolicy": "Always",
			"ConfigGenerationPolicy": "Auto"
		}
	],
	"Modules": [
		{
			"Name": "AAA",
			"Type": "DeveloperTool",
			"LoadingPhase": "Default",
			"HasExplicitPlatforms": true,
			"PlatformAllowList": ["Win64"],
			"PlatformDenyList": ["Mac"],
			"TargetAllowList" : ["Editor"],
			"TargetDenyList" : ["Client"],
			"TargetConfigurationAllowList" : ["Test"],
			"TargetConfigurationDenyList" : ["DebugGame"],
			"ProgramAllowList" : ["UnrealInsights"],
			"ProgramDenyList" : ["LiveLinkHub"],
			"GameTargetAllowList" : ["AAA"],
			"GameTargetDenyList" : ["BBB"],
			"AdditionalDependencies" : ["AAA", "BBB"]
		},
		{
			"Name": "BBB",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CCC",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "DDD",
			"Enabled": true,
			"Optional": true,
			"Description": "Ref Description goes here",
			"MarketplaceURL": "Ref MarketplaceURL goes here",
			"PlatformAllowList": ["Win64"],
			"PlatformDenyList": ["Mac"],
			"TargetConfigurationAllowList" : ["Test"],
			"TargetConfigurationDenyList" : ["DebugGame"],
			"TargetAllowList" : ["Editor"],
			"TargetDenyList" : ["Client"],
			"SupportedTargetPlatforms": ["Win64", "Mac"],
			"HasExplicitPlatforms": true,
			"Version": 2
		},
		{
			"Name": "EEE",
			"Enabled": true
		}
	],
	"PreBuildSteps": {
		"PreBuildStepA": [
			"Step 1a",
			"Step 2a",
		]
	},
	"PostBuildSteps": {
		"PostBuildStepA": ["Step 1b"],
		"PostBuildStepB": ["Step 2b"],
	},
	"DisallowedPlugins": [
		{
			"Name": "ABC",
			"Comment": "ABC Reasons"
		},
	]
}
)json");

	// check to make sure all values are parsed the same across RapidJsonUtils an default Json

	FText FailReason;
	FPluginDescriptor Descriptor;
	REQUIRE_MESSAGE(FailReason.ToString(), Descriptor.Read(JsonText, FailReason));

	// check properties were read
	REQUIRE(Descriptor.Version == 8);
	REQUIRE(Descriptor.VersionName == TEXT("0.2"));
	REQUIRE(Descriptor.FriendlyName == TEXT("Test Plugin"));
	REQUIRE(Descriptor.Description == TEXT("Test Plugin for unit testing"));
	REQUIRE(Descriptor.Category == TEXT("Editor"));
	REQUIRE(Descriptor.CreatedBy == TEXT("Epic Games, Inc."));
	REQUIRE(Descriptor.CreatedByURL == TEXT("https://epicgames.com"));
	REQUIRE(Descriptor.DocsURL == TEXT("DocsURL goes here"));
	REQUIRE(Descriptor.MarketplaceURL == TEXT("MarketplaceURL goes here"));
	REQUIRE(Descriptor.SupportURL == TEXT("SupportURL goes here"));
	REQUIRE(Descriptor.EngineVersion == TEXT("5.5"));
	REQUIRE(Descriptor.EditorCustomVirtualPath == TEXT("EditorCustomVirtualPath goes here"));
	REQUIRE(Descriptor.SupportedTargetPlatforms.Num() == 2);
	REQUIRE(Descriptor.SupportedTargetPlatforms[0] == TEXT("Win64"));
	REQUIRE(Descriptor.SupportedTargetPlatforms[1] == TEXT("Mac"));
	REQUIRE(Descriptor.SupportedPrograms.Num() == 2);
	REQUIRE(Descriptor.SupportedPrograms[0] == TEXT("UnrealInsights"));
	REQUIRE(Descriptor.SupportedPrograms[1] == TEXT("UnrealHeaderTool")); // added by CanBeUsedWithUnrealHeaderTool
	REQUIRE(Descriptor.bIsPluginExtension == true);
	REQUIRE(Descriptor.VersePath == TEXT("/VersePath.com"));
	REQUIRE(Descriptor.DeprecatedEngineVersion == TEXT("5.5"));
	REQUIRE(Descriptor.VerseScope == EVerseScope::InternalAPI);
	REQUIRE(Descriptor.VerseVersion == 1);
	REQUIRE(Descriptor.bEnableSceneGraph == true);
	REQUIRE(Descriptor.bEnableSceneGraphAnimation == true);
	REQUIRE(Descriptor.bEnableVerseAssetReflection == true);
	REQUIRE(Descriptor.bEnableIAD == true);
	REQUIRE(Descriptor.EnabledByDefault == EPluginEnabledByDefault::Disabled);
	REQUIRE(Descriptor.bCanContainContent == true);
	REQUIRE(Descriptor.bCanContainVerse == true);
	REQUIRE(Descriptor.bNoCode == true);
	REQUIRE(Descriptor.bIsBetaVersion == true);
	REQUIRE(Descriptor.bIsExperimentalVersion == true);
	REQUIRE(Descriptor.bInstalled == true);
	REQUIRE(Descriptor.bRequiresBuildPlatform == true);
	REQUIRE(Descriptor.bIsHidden == true);
	REQUIRE(Descriptor.bIsSealed == true);
	REQUIRE(Descriptor.bExplicitlyLoaded == true);
	REQUIRE(Descriptor.bHasExplicitPlatforms == true);

	REQUIRE(Descriptor.LocalizationTargets.Num() == 1);
	REQUIRE(Descriptor.LocalizationTargets[0].Name == TEXT("LocalizationTarget AAA"));
	REQUIRE(Descriptor.LocalizationTargets[0].LoadingPolicy == ELocalizationTargetDescriptorLoadingPolicy::Always);
	REQUIRE(Descriptor.LocalizationTargets[0].ConfigGenerationPolicy == ELocalizationConfigGenerationPolicy::Auto);

	REQUIRE(Descriptor.Modules.Num() == 3);
	REQUIRE(Descriptor.Modules[0].Name == TEXT("AAA"));
	REQUIRE(Descriptor.Modules[0].Type == EHostType::DeveloperTool);
	REQUIRE(Descriptor.Modules[0].LoadingPhase == ELoadingPhase::Default);
	REQUIRE(Descriptor.Modules[0].bHasExplicitPlatforms == true);
	REQUIRE(Descriptor.Modules[0].PlatformAllowList.Num() == 1);
	REQUIRE(Descriptor.Modules[0].PlatformAllowList[0] == TEXT("Win64"));
	REQUIRE(Descriptor.Modules[0].PlatformDenyList[0] == TEXT("Mac"));
	REQUIRE(Descriptor.Modules[0].PlatformDenyList.Num() == 1);
	REQUIRE(Descriptor.Modules[0].TargetAllowList.Num() == 1);
	REQUIRE(Descriptor.Modules[0].TargetAllowList[0] == EBuildTargetType::Editor);
	REQUIRE(Descriptor.Modules[0].TargetDenyList.Num() == 1);
	REQUIRE(Descriptor.Modules[0].TargetDenyList[0] == EBuildTargetType::Client);
	REQUIRE(Descriptor.Modules[0].TargetConfigurationAllowList.Num() == 1);
	REQUIRE(Descriptor.Modules[0].TargetConfigurationAllowList[0] == EBuildConfiguration::Test);
	REQUIRE(Descriptor.Modules[0].TargetConfigurationDenyList.Num() == 1);
	REQUIRE(Descriptor.Modules[0].TargetConfigurationDenyList[0] == EBuildConfiguration::DebugGame);
	REQUIRE(Descriptor.Modules[0].ProgramAllowList.Num() == 1);
	REQUIRE(Descriptor.Modules[0].ProgramAllowList[0] == TEXT("UnrealInsights"));
	REQUIRE(Descriptor.Modules[0].ProgramDenyList.Num() == 1);
	REQUIRE(Descriptor.Modules[0].ProgramDenyList[0] == TEXT("LiveLinkHub"));
	REQUIRE(Descriptor.Modules[0].GameTargetAllowList.Num() == 1);
	REQUIRE(Descriptor.Modules[0].GameTargetAllowList[0] == TEXT("AAA"));
	REQUIRE(Descriptor.Modules[0].GameTargetDenyList.Num() == 1);
	REQUIRE(Descriptor.Modules[0].GameTargetDenyList[0] == TEXT("BBB"));
	REQUIRE(Descriptor.Modules[0].AdditionalDependencies.Num() == 2);
	REQUIRE(Descriptor.Modules[0].AdditionalDependencies[0] == TEXT("AAA"));
	REQUIRE(Descriptor.Modules[0].AdditionalDependencies[1] == TEXT("BBB"));

	REQUIRE(Descriptor.Plugins.Num() == 2);
	REQUIRE(Descriptor.Plugins[0].Name == TEXT("DDD"));
	REQUIRE(Descriptor.Plugins[0].bEnabled == true);
	REQUIRE(Descriptor.Plugins[0].bOptional == true);
	REQUIRE(Descriptor.Plugins[0].Description == TEXT("Ref Description goes here"));
	REQUIRE(Descriptor.Plugins[0].MarketplaceURL == TEXT("Ref MarketplaceURL goes here"));
	REQUIRE(Descriptor.Plugins[0].PlatformAllowList.Num() == 1);
	REQUIRE(Descriptor.Plugins[0].PlatformAllowList[0] == TEXT("Win64"));
	REQUIRE(Descriptor.Plugins[0].PlatformDenyList[0] == TEXT("Mac"));
	REQUIRE(Descriptor.Plugins[0].PlatformDenyList.Num() == 1);
	REQUIRE(Descriptor.Plugins[0].TargetAllowList.Num() == 1);
	REQUIRE(Descriptor.Plugins[0].TargetAllowList[0] == EBuildTargetType::Editor);
	REQUIRE(Descriptor.Plugins[0].TargetDenyList.Num() == 1);
	REQUIRE(Descriptor.Plugins[0].TargetDenyList[0] == EBuildTargetType::Client);
	REQUIRE(Descriptor.Plugins[0].TargetConfigurationAllowList.Num() == 1);
	REQUIRE(Descriptor.Plugins[0].TargetConfigurationAllowList[0] == EBuildConfiguration::Test);
	REQUIRE(Descriptor.Plugins[0].TargetConfigurationDenyList.Num() == 1);
	REQUIRE(Descriptor.Plugins[0].TargetConfigurationDenyList[0] == EBuildConfiguration::DebugGame);

#if WITH_EDITOR
	REQUIRE(Descriptor.PreBuildSteps.HostPlatformToCommands.Num() == 1);
	REQUIRE(Descriptor.PreBuildSteps.HostPlatformToCommands.Contains(TEXT("PreBuildStepA")));
	const TArray<FString>& PreBuildStepsA = Descriptor.PreBuildSteps.HostPlatformToCommands[TEXT("PreBuildStepA")];
	REQUIRE(PreBuildStepsA.Num() == 2);
	REQUIRE(PreBuildStepsA[0] == TEXT("Step 1a"));
	REQUIRE(PreBuildStepsA[1] == TEXT("Step 2a"));
	REQUIRE(Descriptor.PostBuildSteps.HostPlatformToCommands.Num() == 2);
#endif

	REQUIRE(Descriptor.DisallowedPlugins.Num() == 1);
	REQUIRE(Descriptor.DisallowedPlugins[0].Name == TEXT("ABC"));
#if WITH_EDITOR
	REQUIRE(Descriptor.DisallowedPlugins[0].Comment == TEXT("ABC Reasons"));
#endif

}

TEST_CASE_NAMED(FProjectsLoadingProjectDescriptorTests, "System::Engine::Projects::Loading::ProjectDescriptor", "[ApplicationContextMask][SmokeFilter]")
{
	const TCHAR JsonText[] = TEXT(R"json(
{
	"FileVersion": 3,
	"EngineAssociation": "EngineAssociation goes here",
	"Category": "Category goes here",
	"Description": "Description goes here",
	"Enterprise": true,
	"DisableEnginePluginsByDefault": true,
	"AdditionalPluginDirectories": ["Content"],
	"AdditionalRootDirectories": ["/RootDir"],
	"TargetPlatforms": ["Windows", "Linux"],
	"EpicSampleNameHash": 1234,
	"Modules": [
		{
			"Name": "ModuleA",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "PluginA",
			"Enabled": true
		}
	],
	"PreBuildSteps": {
		"PreBuildStepA": [
			"Step 1a",
			"Step 2a",
		]
	},
	"PostBuildSteps": {
		"PostBuildStepA": ["Step 1b"],
		"PostBuildStepB": ["Step 2b"],
	},	
})json");


	FProjectDescriptor Descriptor;

	using namespace UE::Json;

	TValueOrError<FDocument, FParseError> ParseResult = Parse(JsonText);
	FString ParseErrorMessage = ParseResult.HasError() ? ParseResult.GetError().CreateMessage(JsonText) : FString();
	REQUIRE_MESSAGE(ParseErrorMessage, ParseResult.HasValue());

	TOptional<FConstObject> RootObject = GetRootObject(ParseResult.GetValue());
	REQUIRE(RootObject);

	FText FailReason;
	REQUIRE_MESSAGE(FailReason.ToString(), FProjectsLoadingPluginDescriptorTestsHelper::Read(Descriptor, *RootObject, "UnitTest", FailReason));

	REQUIRE(Descriptor.EngineAssociation == TEXT("EngineAssociation goes here"));
	REQUIRE(Descriptor.Category == TEXT("Category goes here"));
	REQUIRE(Descriptor.Description == TEXT("Description goes here"));
	REQUIRE(Descriptor.bIsEnterpriseProject == true);
	REQUIRE(Descriptor.bDisableEnginePluginsByDefault == true);
	REQUIRE(Descriptor.GetAdditionalPluginDirectories().Num() == 1);
#if WITH_EDITOR	
	REQUIRE(Descriptor.GetAdditionalRootDirectories().Num() == 1);
#endif
	REQUIRE(Descriptor.TargetPlatforms.Num() >= 2);
	REQUIRE(Descriptor.EpicSampleNameHash == 1234);

	// their contents are already tested in the plugin loading test
	REQUIRE(Descriptor.Modules.Num() == 1);
	REQUIRE(Descriptor.Plugins.Num() == 1);
	REQUIRE(Descriptor.PreBuildSteps.HostPlatformToCommands.Num() == 1);
	REQUIRE(Descriptor.PostBuildSteps.HostPlatformToCommands.Num() == 2);


}

TEST_CASE_NAMED(FProjectsLoadingRapidJsonPluginLoadingCaseInsensitiveKeys, "System::Engine::Projects::Loading::RapidJsonPluginLoading::CaseInsensitiveKeys", "[ApplicationContextMask][SmokeFilter]")
{
	const TCHAR JsonText[] = TEXT(R"json(
{
	"aaa": 1,
	"bBb": 2,
	"CCC": 3,
})json");


	FProjectDescriptor Descriptor;

	using namespace UE::Json;
	using namespace UE::Projects::Private;

	TValueOrError<FDocument, FParseError> RapidJsonDocument = UE::Json::Parse(JsonText);
	FString ParseErrorMessage = RapidJsonDocument.HasError() ? RapidJsonDocument.GetError().CreateMessage(JsonText) : TEXT("No Error");
	REQUIRE_MESSAGE(ParseErrorMessage, RapidJsonDocument.HasValue());

	TOptional<UE::Json::FConstObject> RapidRoot = UE::Json::GetRootObject(RapidJsonDocument.GetValue());
	REQUIRE(RapidRoot.IsSet());

	int32 AAA = -1;
	int32 BBB = -1;
	int32 CCC = -1;

	REQUIRE(TryGetNumberField(*RapidRoot, TEXT("Aaa"), AAA));
	REQUIRE(TryGetNumberField(*RapidRoot, TEXT("Bbb"), BBB));
	REQUIRE(TryGetNumberField(*RapidRoot, TEXT("Ccc"), CCC));

	REQUIRE(AAA == 1);
	REQUIRE(BBB == 2);
	REQUIRE(CCC == 3);
}

#endif // WITH_TESTS
