// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace UnrealBuildTool.Rules
{
	public class Projects : ModuleRules
	{
		public Projects(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"Json",
				}
			);

			// Monolithic and non-editor builds compile in the enabled/disabled plugins. Non-monolithic editor builds save them in the plugin receipt to avoid invalidating the shared build environment.
			if ((Target.Type == TargetType.Editor) && (Target.BuildEnvironment == TargetBuildEnvironment.Shared) && (Target.LinkType != TargetLinkType.Monolithic))
			{
				PublicDefinitions.Add("READ_TARGET_ENABLED_PLUGINS_FROM_RECEIPT=1");

				// Include DesktopPlatform to work with the target receipt which contains the enabled/disabled plugins
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"DesktopPlatform"
					}
				);
			}
			else
			{
				PublicDefinitions.Add("READ_TARGET_ENABLED_PLUGINS_FROM_RECEIPT=0");

				List<string> EnabledPluginStrings = new List<string>();
				foreach(string EnablePlugin in Target.EnablePlugins)
				{
					EnabledPluginStrings.Add($"TEXT(\"{EnablePlugin}\")");
				}
				EnabledPluginStrings.Sort();
				PrivateDefinitions.Add(String.Format("UBT_TARGET_ENABLED_PLUGINS={0}", String.Join(", ", EnabledPluginStrings)));

				List<string> DisabledPluginStrings = new List<string>();
				foreach(string DisablePlugin in Target.DisablePlugins)
				{
					DisabledPluginStrings.Add($"TEXT(\"{DisablePlugin}\")");
				}
				DisabledPluginStrings.Sort();
				PrivateDefinitions.Add(String.Format("UBT_TARGET_DISABLED_PLUGINS={0}", String.Join(", ", DisabledPluginStrings)));
			}

			if (Target.bIncludePluginsForTargetPlatforms)
			{
				PublicDefinitions.Add("LOAD_PLUGINS_FOR_TARGET_PLATFORMS=1");
			}
			else
			{
				PublicDefinitions.Add("LOAD_PLUGINS_FOR_TARGET_PLATFORMS=0");
			}

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
