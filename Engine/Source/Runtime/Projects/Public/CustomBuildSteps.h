// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Serialization/JsonWriter.h"

class FJsonObject;

/**
 * Descriptor for projects. Contains all the information contained within a .uproject file.
 */
struct FCustomBuildSteps
{
	/** Mapping from host platform to list of commands */
	TMap<FString, TArray<FString>> HostPlatformToCommands;

	/** Whether this object is empty */
	PROJECTS_API bool IsEmpty() const;

	/** Reads the descriptor from the given JSON object */
	PROJECTS_API void Read(const FJsonObject& Object, const FString& FieldName);

	/** Writes the CustomBuildSteps to JSON */
	PROJECTS_API void Write(TJsonWriter<>& Writer, const FString& FieldName) const;

	/** Updates the given json object with this CustomBuildSteps */
	PROJECTS_API void UpdateJson(FJsonObject& JsonObject, const FString& FieldName) const;
};
