// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "PluginDescriptor.h"
#include "Containers/VersePathFwd.h"
#include "Templates/SharedPointer.h"

struct FProjectDescriptor;
class FJsonObject;
struct FPluginReferenceDescriptor;

/**
 * Enum for where a plugin is loaded from
 */
enum class EPluginLoadedFrom
{
	/** Plugin is built-in to the engine */
	Engine,

	/** Project-specific plugin, stored within a game project directory */
	Project
};

/**
 * Enum for the type of a plugin
 */
enum class EPluginType
{
	/** Plugin is built-in to the engine */
	Engine,

	/** Standard enterprise plugin */
	Enterprise,

	/** Project-specific plugin, stored within a game project directory */
	Project,

	/** Plugin found in an external directory (found in an AdditionalPluginDirectory listed in the project file, or referenced on the command line) */
	External,

	/** Project-specific mod plugin */
	Mod,
};

/**
 * Enum describing a source of EPluginType::External plugins
 */
enum class EPluginExternalSource
{
	/** .uproject `AdditionalPluginDirectories` field */
	ProjectDescriptor,

	/** Passed via `-plugin=` command line switch. */
	CommandLine,

	/** Inherited from `UE_ADDITIONAL_PLUGIN_PATHS` environment variable. */
	Environment,

	/** None of the above (e.g. added explicitly by external code). */
	Other,
};

/**
 * Combination of a directory searched for external plugins, and its configuration source.
 */
struct FExternalPluginPath
{
	FString Path;
	EPluginExternalSource Source;

	inline bool operator==(const FExternalPluginPath& Other) const
	{
		return (Other.Path == Path) && (Other.Source == Source);
	}

	friend uint32 GetTypeHash(const FExternalPluginPath& ExternalPluginPath)
	{
		return HashCombine(GetTypeHash(ExternalPluginPath.Path), GetTypeHash(ExternalPluginPath.Source));
	}
};

/**
 * Simple data structure that is filled when querying information about plug-ins.
 */
struct FPluginStatus
{
	/** The name of this plug-in. */
	FString Name;

	/** Path to plug-in directory on disk. */
	FString PluginDirectory;

	/** True if plug-in is currently enabled. */
	bool bIsEnabled;

	/** Where the plugin was loaded from */
	EPluginLoadedFrom LoadedFrom;

	/** The plugin descriptor */
	FPluginDescriptor Descriptor;
};


/**
 * Information about an enabled plugin.
 */
class IPlugin : public TSharedFromThis<IPlugin>
{
public:
	/* Virtual destructor */
	virtual ~IPlugin()= default;

	/**
	 * Gets the plugin name.
	 *
	 * @return Name of the plugin.
	 */
	virtual const FString& GetName() const = 0;

	/**
	 * Return plugin friendly name if available or the same name as GetName() otherwise.
	 */
	virtual const FString& GetFriendlyName() const = 0;

	/**
	 * Get a filesystem path to the plugin's descriptor
	 *
	 * @return Filesystem path to the plugin's descriptor.
	 */
	virtual const FString& GetDescriptorFileName() const = 0;

	/**
	 * Get a filesystem path to the plugin's directory.
	 *
	 * @return Filesystem path to the plugin's base directory.
	 */
	virtual FString GetBaseDir() const = 0;

	/**
	 * Get a filesystem path to the plugin's directory.
	 *
	 * @return Filesystem path to the plugin's base directory.
	 */
	virtual TArray<FString> GetExtensionBaseDirs() const = 0;

	/**
	 * Get a filesystem path to the plugin's content directory.
	 *
	 * @return Filesystem path to the plugin's content directory.
	 */
	virtual FString GetContentDir() const = 0;

	/**
	 * Get the virtual root path for assets.
	 *
	 * @return The mounted root path for assets in this plugin's content folder; typically /PluginName/.
	 */
	virtual FString GetMountedAssetPath() const = 0;

	/**
	 * Gets the type of a plugin
	 *
	 * @return The plugin type
	 */
	virtual EPluginType GetType() const = 0;

	/**
	 * Determines if the plugin is enabled.
	 *
	 * @return True if the plugin is currently enabled.
	 */
	virtual bool IsEnabled() const = 0;

	/**
	 * Determines if the plugin is enabled by default.
	 *
	 * @return True if the plugin is currently enabled by default.
	 */
	virtual bool IsEnabledByDefault(bool bAllowEnginePluginsEnabledByDefault) const = 0;

	/**
	 * Determines if the plugin is mounted.
	 *
	 * @return True if the plugin is currently mounted.
	 */
	virtual bool IsMounted() const = 0;

	/**
	 * Determines if the plugin is should be displayed in-editor for the user to enable/disable freely.
	 *
	 * @return True if the plugin should be hidden.
	 */
	virtual bool IsHidden() const = 0;

	/**
	 * Determines if the plugin can contain content.
	 *
	 * @return True if the plugin can contain content.
	 */
	virtual bool CanContainContent() const = 0;

	/**
	 * Determines if the plugin can contain Verse code.
	 *
	 * @return True if the plugin can contain Verse code.
	 */
	virtual bool CanContainVerse() const = 0;

	/**
	 * Gets the Verse path to the root of the plugin's content directory
	 *
	 * @return Verse path to the root of the plugin's content directory
	 */
	virtual const FString& GetVersePath() const = 0;

	/**
	 * Gets the engine version that this plugin was deprecated.  If unset
	 * then this plugin is not deprecated.
	 *
	 * @return Deprecated engine version.  This may be an empty string.
	 */
	virtual const FString& GetDeprecatedEngineVersion() const = 0;

	/**
	 * Return the Verse language version to compile the plugin with (if unspecified, the latest stable version is used)
	 */
	virtual TOptional<uint32> GetVerseVersion() const = 0;

	/**
	 * Return whether Scene Graph is enabled or not. This impacts which Verse asset digest is generated.
	 */
	virtual bool IsSceneGraphEnabled() const = 0;

	/**
	 * Return whether Scene Graph Animation is enabled or not. This impacts which Verse asset digest is generated.
	 */
	virtual bool IsSceneGraphAnimationEnabled() const = 0;

	/**
	 * Returns true if this is a root plugin. A root plugin is meant as a collection of other plugins.
	 * Root plugins should only have plugin dependencies and be used to designate some group of plugins.
	 */
	virtual bool IsRootPlugin() const = 0;

	/**
	 * Returns the plugin's location
	 *
	 * @return Where the plugin was loaded from
	 */
	virtual EPluginLoadedFrom GetLoadedFrom() const = 0;

	/**
	 * Gets the plugin's descriptor
	 *
	 * @return Reference to the plugin's descriptor
	 */
	virtual const FPluginDescriptor& GetDescriptor() const = 0;

	/**
	 * Updates the plugin's descriptor
	 *
	 * @param NewDescriptor The new plugin descriptor
	 * @param OutFailReason The error message if the plugin's descriptor could not be updated
	 * @return True if the descriptor was updated, false otherwise. 
	 */ 
	virtual bool UpdateDescriptor(const FPluginDescriptor& NewDescriptor, FText& OutFailReason) = 0;

#if WITH_EDITOR
	/**
	 * Gets the cached plugin descriptor json
	 *
	 * @return Reference to the cached plugin descriptor json
	 */
	virtual const TSharedPtr<FJsonObject>& GetDescriptorJson() = 0;
#endif // WITH_EDITOR
};

/**
 * PluginManager manages available code and content extensions (both loaded and not loaded).
 */
class IPluginManager
{
public:
	virtual ~IPluginManager() = default;

	/** 
	 * Updates the list of plugins.
	 */
	virtual void RefreshPluginsList() = 0;

	/**
	 * Adds a single plugin to the list of plugins. Faster than refreshing all plugins with RefreshPluginsList() when you only want to add one. Does nothing if already in the list.
	 * 
	 * @return True if the plugin was added or already in the list. False if it failed to load.
	 */
	virtual bool AddToPluginsList(const FString& PluginFilename, FText* OutFailReason = nullptr) = 0;

	/**
	 * Remove a single plugin from the list of plugins.
	 *
	 * @return True if the plugin was not in the list. False if it can't be removed (see OutFailReason).
	 */
	virtual bool RemoveFromPluginsList(const FString& PluginFilename, FText* OutFailReason = nullptr) = 0;

	/**
	 * Loads all plug-ins
	 *
	 * @param	LoadingPhase	Which loading phase we're loading plug-in modules from.  Only modules that are configured to be
	 *							loaded at the specified loading phase will be loaded during this call.
	 */
	virtual bool LoadModulesForEnabledPlugins( const ELoadingPhase::Type LoadingPhase ) = 0;

	/** Returns the highest loading phase that has so far completed */
	virtual ELoadingPhase::Type GetLastCompletedLoadingPhase() const = 0;

	/**
	 * Callback for when modules for when LoadModulesForEnabledPlugins() completes loading for a specific phase.
	 */
	DECLARE_EVENT_TwoParams(IPluginManager, FLoadingModulesForPhaseEvent, ELoadingPhase::Type /*LoadingPhase*/, bool /*bSuccess*/);
	virtual FLoadingModulesForPhaseEvent& OnLoadingPhaseComplete() = 0;

	/**
	 * Get the localization paths for all enabled plugins.
	 *
	 * @param	OutLocResPaths	Array to populate with the localization paths for all enabled plugins.
	 */
	virtual void GetLocalizationPathsForEnabledPlugins( TArray<FString>& OutLocResPaths ) = 0;

	/** Delegate type for mounting content paths.  Used internally by FPackageName code. */
	DECLARE_DELEGATE_TwoParams( FRegisterMountPointDelegate, const FString& /* Root content path */, const FString& /* Directory name */ );

	/**
	 * Sets the delegate to call to register a new content mount point.  This is used internally by the plug-in manager system
	 * and should not be called by you.  This is registered at application startup by FPackageName code in CoreUObject.
	 *
	 * @param	Delegate	The delegate to that will be called when plug-in manager needs to register a mount point
	 */
	virtual void SetRegisterMountPointDelegate( const FRegisterMountPointDelegate& Delegate ) = 0;

	/**
	 * Sets the delegate to call to unregister a new content mount point.  This is used internally by the plug-in manager system
	 * and should not be called by you.  This is registered at application startup by FPackageName code in CoreUObject.
	 *
	 * @param	Delegate	The delegate to that will be called when plug-in manager needs to unregister a mount point
	 */
	virtual void SetUnRegisterMountPointDelegate( const FRegisterMountPointDelegate& Delegate ) = 0;

	/**
	 * Checks if all the required plug-ins are available. If not, will present an error dialog the first time a plug-in is loaded or this function is called.
	 *
	 * @returns true if all the required plug-ins are available.
	 */
	virtual bool AreRequiredPluginsAvailable() = 0;

	/** Delegate type for allowing higher levels systems to provide additional context for why a package was skipped during loading */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FGetExplanationForUnavailablePackageWithPluginInfoDelegate, const FString& /*Unavailable package*/, class IPlugin* /*Plugin or null if not found*/, FStringBuilderBase& /*InOutStringBuilder*/);

	/**
	 * Accessor for adding/removing FGetExplanationForUnavailablePackageWithPluginInfoDelegate delegates.
	 * 
	 * @returns the FGetExplanationForUnavailablePackageWithPluginInfoDelegate to which callbacks can be bound
	 */
	virtual FGetExplanationForUnavailablePackageWithPluginInfoDelegate& GetExplanationForUnavailablePackageWithPluginInfoDelegate() = 0;

	/**
	 * Attempts to fill out InOutExplanation with information from various sources (registered via GetExplanationForUnavailablePackageWithPluginInfoDelegate)
	 * about why UnavailablePackageName is not available
	 * 
	 * @param UnavailablePackageName the name of the package about which to get information
	 * @param InOutExplanation a stringbuilder to which to append explanatory information
	 */
	virtual	void GetExplanationForUnavailablePackage(const FName& UnavailablePackageName, FStringBuilderBase& InOutExplanation) = 0;

#if !IS_MONOLITHIC
	/** 
	 * Checks whether modules for the enabled plug-ins are up to date.
	 *
	 * @param OutIncompatibleModules Array to receive a list of incompatible module names.
	 * @param OutIncompatibleEngineModules Array to receive a list of incompatible engine module names.
	 * @returns true if the enabled plug-in modules are up to date.
	 */
	virtual bool CheckModuleCompatibility( TArray<FString>& OutIncompatibleModules, TArray<FString>& OutIncompatibleEngineModules ) = 0;
#endif

	/**
	 * Finds information for a plugin.
	 *
	 * @return	 Pointer to the plugin's information, or nullptr.
	 */
	virtual TSharedPtr<IPlugin> FindPlugin(const FStringView Name) = 0;
	virtual TSharedPtr<IPlugin> FindPlugin(const ANSICHAR* Name) = 0;

	virtual TSharedPtr<IPlugin> FindPluginFromPath(const FString& PluginPath) = 0;
	virtual TSharedPtr<IPlugin> FindPluginFromDescriptor(const FPluginReferenceDescriptor& PluginDesc) = 0;

	/**
	 * Finds information for an enabled plugin.
	 *
	 * @return	 Pointer to the enabled plugin's information, or nullptr if not enabled or can't be found.
	 */
	virtual TSharedPtr<IPlugin> FindEnabledPlugin(const FStringView Name) = 0;
	virtual TSharedPtr<IPlugin> FindEnabledPlugin(const ANSICHAR* Name) = 0;

	virtual TSharedPtr<IPlugin> FindEnabledPluginFromPath(const FString& PluginPath) = 0;
	virtual TSharedPtr<IPlugin> FindEnabledPluginFromDescriptor(const FPluginReferenceDescriptor& PluginDesc) = 0;

	/**
	 * Returns true when the given plugin can be enabled for the current running exe.
	 */
	virtual bool CanEnablePluginInCurrentTarget(const FStringView Name) = 0;
	virtual bool CanEnablePluginInCurrentTarget(const ANSICHAR* Name) = 0;

	/** 
	 * Finds all plugin descriptors underneath a given directory (recursively)
	 * @param Directory Search folder
	 * @param OutPluginFilePaths Receives found plugin descriptor file paths
	 */
	virtual void FindPluginsUnderDirectory(const FString& Directory, TArray<FString>& OutPluginFilePaths) = 0;

	/**
	 * Gets an array of all the enabled plugins.
	 *
	 * @return	Array of the enabled plugins.
	 */
	virtual TArray<TSharedRef<IPlugin>> GetEnabledPlugins() = 0;

	/**
	 * Gets an array of all enabled plugins that can have content.
	 *
	 * @return	Array of plugins with IsEnabled() and CanContainContent() both true.
	 */
	virtual TArray<TSharedRef<IPlugin>> GetEnabledPluginsWithContent() const = 0;

	/**
	 * Gets an array of all enabled plugins that can have Verse code.
	 *
	 * @return	Array of plugins with IsEnabled() and CanContainVerse() both true.
	 */
	virtual TArray<TSharedRef<IPlugin>> GetEnabledPluginsWithVerse() const = 0;

	/**
	 * Gets an array of all enabled plugins that can have content or Verse code.
	 */
	virtual TArray<TSharedRef<IPlugin>> GetEnabledPluginsWithContentOrVerse() const = 0;

	/**
	 * Gets an array of all the discovered plugins.
	 *
	 * @return	Array of the discovered plugins.
	 */
	virtual TArray<TSharedRef<IPlugin>> GetDiscoveredPlugins() = 0;

	/**
	 * Returns the set of plugins compiled into the current application and enabled at startup.
	 * For internal use only.
	 * TODO: Needs improved for the GameFeaturePlugin system to one day remove the concept of LoadedAsBuiltIn
	 * 
	 * @return set of plugin names.
	 */
	UE_INTERNAL virtual const TSet<FString>& GetPluginsEnabledAtStartup() const = 0;

#if WITH_EDITOR
	/**
	 * Returns the set of built-in plugin names
	 */
	virtual const TSet<FString>& GetBuiltInPluginNames() const = 0;

	/**
	 * Returns the plugin that owns the specified module, if any
	 */
	virtual TSharedPtr<IPlugin> GetModuleOwnerPlugin(FName ModuleName) const = 0;
#endif //WITH_EDITOR

	/**
	 * Stores the specified path, utilizing it in future search passes when 
	 * searching for available plugins. Optionally refreshes the manager after 
	 * the new path has been added.
	 * 
	 * @param  ExtraDiscoveryPath	The path you want searched for additional plugins.
	 * @param  bRefresh				Signals the function to refresh the plugin database after the new path has been added
	 * @return Whether the plugin search path was modified
	 */
	virtual bool AddPluginSearchPath(const FString& ExtraDiscoveryPath, bool bRefresh = true) = 0;

	/**
	 * Removes the specified path from consideration for available plugins.
	 * Optionally refreshes the manager after the path has been removed.
	 *
	 * @param  PathToRemove			The path no longer searched for additional plugins.
	 * @param  bRefresh				Signals the function to refresh the plugin database after the path has been removed
	 * @return Whether the plugin search path was modified
	 */
	virtual bool RemovePluginSearchPath(const FString& PathToRemove, bool bRefresh = true) = 0;

	/**
	 * Returns the list of extra directories that are recursively searched for plugins (aside from the engine and project plugin directories).
	 * NOTE: You may also want to check IProjectManager::GetAdditionalPluginDirectories, which are not included here!
	 */
	virtual const TSet<FString>& GetAdditionalPluginSearchPaths() const = 0;

	/**
	 * Similar to GetAdditionalPluginSearchPaths, but also returns configuration sources for each path.
	 * Unlike GetAdditionalPluginSearchPaths, this DOES include AdditionalPluginDirectories from the uproject descriptor.
	 */
	virtual void GetExternalPluginSources(TSet<FExternalPluginPath>& OutPluginSources) const = 0;

	/**
	 * Gets an array of plugins that loaded their own content pak file
	 */
	virtual TArray<TSharedRef<IPlugin>> GetPluginsWithPakFile() const = 0;

	/**
	 * Event signature for being notified that a new plugin has been mounted
	 */
	DECLARE_EVENT_OneParam(IPluginManager, FNewPluginMountedEvent, IPlugin&);

	/**
	 * Event signature for being notified that a new plugin has been created
	 */
	virtual FNewPluginMountedEvent& OnNewPluginCreated() = 0;

	/**
	 * Event for being notified that a new plugin has been mounted
	 */
	virtual FNewPluginMountedEvent& OnNewPluginMounted() = 0;

	/**
	 * Event for being notified that a new plugin and its content have been mounted
	 */
	virtual FNewPluginMountedEvent& OnNewPluginContentMounted() = 0;

	/**
	 * Event for being notified that a plugin has been edited
	 */
	virtual FNewPluginMountedEvent& OnPluginEdited() = 0;

	/**
	* Event for being notified that a plugin has been unmounted
	*/
	virtual FNewPluginMountedEvent& OnPluginUnmounted() = 0;

	/**
	 * Marks a newly created plugin as enabled, mounts its content and tries to load its modules
	 */
	virtual void MountNewlyCreatedPlugin(const FString& PluginName) = 0;

	/**
	 * Marks an explicitly loaded plugin as enabled, mounts its content and tries to load its modules.
	 * These plugins are not loaded implicitly, but instead wait for this function to be called.
	 * 
	 * @note Call MountExplicitlyLoadedPluginLocalizationData if you also want to load any localization data for this plugin.
	 */
	virtual bool MountExplicitlyLoadedPlugin(const FString& PluginName, ELoadingPhase::Type MaxLoadingPhase = ELoadingPhase::Max) = 0;
	virtual bool MountExplicitlyLoadedPlugin_FromFileName(const FString& PluginFileName) = 0;
	virtual bool MountExplicitlyLoadedPlugin_FromDescriptor(const FPluginReferenceDescriptor& PluginDescriptor) = 0;

	/**
	 * Start loading localization data for an explicitly loaded plugin that has previously been mounted via one of the MountExplicitlyLoadedPlugin functions.
	 * @note Localization data is ref-counted, so this should be paired with a call to UnmountExplicitlyLoadedPluginLocalizationData.
	 * 
	 * @param PluginName The name of the plugin to load the localization data for.
	 * @param Notification An optional notification function to call when the localization data load has finished (from any thread), or immediately if there was no localization data to load.
	 *            bLoadedLocalization corresponds to the return value of this function.
	 * 
	 * @return True if localization data started to load, or false if the plugin was missing or had no localization data to load.
	 */
	virtual bool MountExplicitlyLoadedPluginLocalizationData(const FString& PluginName, TUniqueFunction<void(bool bLoadedLocalization, const FString& PluginName)>&& Notification = nullptr) = 0;

	/**
	 * Start unloading localization data for an explicitly loaded plugin that had its localization data mounted via MountExplicitlyLoadedPluginLocalizationData.
	 * @note Localization data is ref-counted, so this should be paired with a call to MountExplicitlyLoadedPluginLocalizationData.
	 * @note Localization data is also automatically (and regardless of ref-count) unloaded when calling UnmountExplicitlyLoadedPlugin.
	 *
	 * @param PluginName The name of the plugin to unload the localization data for.
	 * @param Notification An optional notification function to call when the localization data unload has finished (from any thread), or immediately if there was no localization data to unload.
	 *            bUnloadedLocalization corresponds to the return value of this function.
	 *
	 * @return True if localization data started to unload, or false if the plugin was missing or had no localization data to unload.
	 */
	virtual bool UnmountExplicitlyLoadedPluginLocalizationData(const FString& PluginName, TUniqueFunction<void(bool bUnloadedLocalization, const FString& PluginName)>&& Notification = nullptr) = 0;

	/**
	 * Marks an explicitly loaded plugin as disabled, unmounts its content (does not work on plugins with compiled modules).
	 */
	virtual bool UnmountExplicitlyLoadedPlugin(const FString& PluginName, FText* OutReason) = 0;
	virtual bool UnmountExplicitlyLoadedPlugin(const FString& PluginName, FText* OutReason, bool bAllowUnloadCode) = 0;

	/**
	 * Tries to get a list of plugin dependencies for a given plugin. Returns false if the plugin provided was not found
	 */
	virtual bool GetPluginDependencies(const FString& PluginName, TArray<FPluginReferenceDescriptor>& PluginDependencies) = 0;
	virtual bool GetPluginDependencies_FromFileName(const FString& PluginFileName, TArray<FPluginReferenceDescriptor>& PluginDependencies) = 0;
	virtual bool GetPluginDependencies_FromDescriptor(const FPluginReferenceDescriptor& PluginDescriptor, TArray<FPluginReferenceDescriptor>& PluginDependencies) = 0;

	/**
	* Does a reverse lookup to try to figure out what the UObject package name is for a plugin
	*/
	virtual FName PackageNameFromModuleName(FName ModuleName) = 0;

	/**
	* Enumerates all mounted plug-ins to find mount points that the Verse path maps to.
	* 
	* @param VersePath The full Verse path to check against the plug-in Verse path.
	* @param Callback Function to call for each plug-in that maps to the Verse path. Return false to stop enumerating.
	*/
	UE_INTERNAL virtual void EnumerateVersePathMountPoints(const UE::Core::FVersePath& VersePath, TFunctionRef<bool(FStringView /*MountedAssetPath*/, FStringView /*LeafPath*/)> Callback) = 0;

	/**
	 * Determines if a content-only project requires a temporary target due to having a plugin enabled
	 *
	 * @param ProjectDescriptor The project being built
	 * @param Platform The platform the target is being built for
	 * @param Configuration The configuration being built
	 * @param TargetType The type of target being built
	 * @param OutReason If a temporary target is required, receives a message indicating why
	 * @return True if the project requires a temp target to be generated
	 */
	virtual bool RequiresTempTargetForCodePlugin(const FProjectDescriptor* ProjectDescriptor, const FString& Platform, EBuildConfiguration Configuration, EBuildTargetType TargetType, FText& OutReason) = 0;

	/**
	 * Scans a set of given plugins and adds them to the passed in ConfigSystem so that the runtime can 
	 * load faster without needing to scan all plugins looking for config/paks
	 *
	 * @param ConfigSystem The config system to insert settings into
	 * @param EngineIniName The name of the engine ini file in ConfigSystem
	 * @param PlatformName The name of the platform this config is made for
	 * @param StagedPluginsFile A path to a file that contains all plugins that have been staged, and should be evaluated
	 */
	UE_DEPRECATED(5.6, "This has never worked and is unsafe. Do not call this function")
	virtual bool IntegratePluginsIntoConfig(FConfigCacheIni& ConfigSystem, const TCHAR* EngineIniName, const TCHAR* PlatformName, const TCHAR* StagedPluginsFile) { return false; }

	/**
	* Set root directories for where to find binaries for plugins.
	*/
	virtual void SetBinariesRootDirectories(const FString& EngineBinariesRootDir, const FString& ProjectBinariesRootDir) = 0;

	/**
	* If preload binaries is set all plugin binaries will be loaded in an early Loading phase.
	* This is a temporary solution to work around issues with pak/iostore for modular builds
	*/ 
	virtual void SetPreloadBinaries() = 0;
	virtual bool GetPreloadBinaries() = 0;

	/**
	 * Internal API for handling plugin garbage collection in a batch-friendly way.
	 */
	UE_INTERNAL virtual void SuppressPluginUnloadGC() = 0;
	UE_INTERNAL virtual void ResumePluginUnloadGC() = 0;

	/**
	 * Callback for when leaked packages in plugins have been found after unmount.
	 */
	using FLeakedPackageCountPerPluginMap = TMap<FString, int32>;
	DECLARE_EVENT_OneParam(IPluginManager, FLeakedPackagesFoundEvent, const FLeakedPackageCountPerPluginMap& /*LeakedPackageCountPerPlugin*/);
	virtual FLeakedPackagesFoundEvent& OnLeakedPackagesFound() = 0;

public:

	/**
	 * Static: Access singleton instance.
	 *
	 * @return	Reference to the singleton object.
	 */
	static PROJECTS_API IPluginManager& Get();
};
