// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "ModuleDescriptor.h"

struct FProjectDescriptor;

/**
 * Simple data structure that is filled when querying information about projects
 */
class FProjectStatus
{
public:

	/** The name of this project */
	FString Name;

	/** The description of this project */
	FString Description;

	/** The UI category of this project */
	FString Category;

	/** True if this project is a sample provided by epic */
	bool bSignedSampleProject;

	/** True if the project is code-based */
	bool bCodeBasedProject;

	/** True if this project needs to be updated */
	bool bRequiresUpdate;

	/** Array of platforms that this project is targeting */
	TArray<FName> TargetPlatforms;

	FProjectStatus()
		: bSignedSampleProject(false)
		, bCodeBasedProject(false)
		, bRequiresUpdate(false)
	{}

	/**
	 * Check to see if the given platform name is supported as a target by the current project
	 * @todo platformcleanup: Make this use IniPlatformName, not TargetPlatformName!! And other related functions in here
	 *
	 * @param	InPlatformName				Name of the platform to target (eg, WindowsClient)
	 * @param	bAllowSupportedIfEmptyList	Consider an empty list to mean that all platforms are supported targets?
	 */
	bool IsTargetPlatformSupported(const FName& InPlatformName, const bool bAllowSupportedIfEmptyList = true) const
	{
		// An empty list is considered the same as supporting all platforms
		return (bAllowSupportedIfEmptyList && TargetPlatforms.Num() == 0) || TargetPlatforms.Contains(InPlatformName);
	}

	/**
	 * Check to see if the given the current project supports all platforms
	 */
	bool SupportsAllPlatforms() const
	{
		return TargetPlatforms.Num() == 0;
	}
};

/**
 * ProjectAndPluginManager manages available code and content extensions (both loaded and not loaded.)
 */
class IProjectManager
{

public:
	virtual ~IProjectManager() = default;

	/**
	 * Static: Access singleton instance
	 *
	 * @return	Reference to the singleton object
	 */
	static PROJECTS_API IProjectManager& Get();

	/**
	 * Gets the current project descriptor.
	 *
	 * @return Pointer to the currently loaded project descriptor. NULL if none is loaded.
	 */
	virtual const FProjectDescriptor* GetCurrentProject() const = 0;

	/**
	 * Loads the specified project file.
	 *
	 * @param ProjectFile - The project file to load.
	 *
	 * @return true if the project was loaded successfully, false otherwise.
	 *
	 * @see GetLoadedGameProjectFile
	 */
	virtual bool LoadProjectFile( const FString& ProjectFile ) = 0;

	/**
	 * Loads all modules for the currently loaded project in the specified loading phase
	 *
	 * @param	LoadingPhase	Which loading phase we're loading modules from.  Only modules that are configured to be
	 *							loaded at the specified loading phase will be loaded during this call.
	 */
	virtual bool LoadModulesForProject( const ELoadingPhase::Type LoadingPhase ) = 0;

	/**
	* Replaces a referenced module with a new name
	*
	* @param	OriginalModuleName	Name of module to replace
	* @param	NewModuleName		Name of replacement module
	*
	*/
	virtual bool SubstituteModule(const FString& OriginalModuleName, const FString& NewModuleName) = 0;

	/**
	 * Callback for when modules for when LoadModulesForProject() completes loading for a specific phase.
	 */
	DECLARE_EVENT_TwoParams(IProjectManager, FLoadingModulesForPhaseEvent, ELoadingPhase::Type /*LoadingPhase*/, bool /*bSuccess*/);
	virtual FLoadingModulesForPhaseEvent& OnLoadingPhaseComplete() = 0;

#if !IS_MONOLITHIC
	/**
	 * Checks if the modules for a project are up to date
	 *
	 * @return	false if UBT needs to be run to recompile modules for a project.
	 */
	virtual bool CheckModuleCompatibility(TArray<FString>& OutIncompatibleModules) = 0;
#endif

	/**
	 * Gets the name of the text file that contains the most recently loaded filename.
	 *
	 * This is NOT the name of the recently loaded .uproject file.
	 *
	 * @return File name.
	 */
	virtual const FString& GetAutoLoadProjectFileName() = 0;

	/**
	 * Sets the project's EpicSampleNameHash (based on its filename) and category, then saves the file to disk.
	 * This marks the project as a sample and fixes its filename so that
	 * it isn't mistaken for a sample if a copy of the file is made.
	 *
	 * @param FilePath				The filepath where the sample project is stored.
	 * @param Category				Category to place the sample in
	 * @param OutFailReason			When returning false, this provides a display reason why the file could not be created.
	 *
	 * @return true if the file was successfully updated and saved to disk
	 */
	virtual bool SignSampleProject(const FString& FilePath, const FString& Category, FText& OutFailReason) = 0;

	/**
	 * Gets status about the specified project
	 *
	 * @param FilePath				The filepath where the project is stored.
	 * @param OutProjectStatus		The status for the project.
	 *
	 * @return	 true if the file was successfully open and read
	 */
	virtual bool QueryStatusForProject(const FString& FilePath, FProjectStatus& OutProjectStatus) const = 0;

	/**
	 * Gets status about the current project
	 *
	 * @param OutProjectStatus		The status for the project.
	 *
	 * @return	 true if the file was successfully open and read
	 */
	virtual bool QueryStatusForCurrentProject(FProjectStatus& OutProjectStatus) const = 0;

	/**
	 * Update the list of supported target platforms for the target project based upon the parameters provided
	 * 
	 * @param	FilePath			The filepath where the project is stored.
	 * @param	InPlatformName		Name of the platform to target (eg, WindowsClient)
	 * @param	bIsSupported		true if the platform should be supported by this project, false if it should not
	 */
	virtual void UpdateSupportedTargetPlatformsForProject(const FString& FilePath, const FName& InPlatformName, const bool bIsSupported) = 0;

	/**
	 * Update the list of supported target platforms for the current project based upon the parameters provided
	 * 
	 * @param	InPlatformName		Name of the platform to target (eg, WindowsClient)
	 * @param	bIsSupported		true if the platform should be supported by this project, false if it should not
	 */
	virtual void UpdateSupportedTargetPlatformsForCurrentProject(const FName& InPlatformName, const bool bIsSupported) = 0;

	/** Clear the list of supported target platforms for the target project */
	virtual void ClearSupportedTargetPlatformsForProject(const FString& FilePath) = 0;

	/** Clear the list of supported target platforms for the current project */
	virtual void ClearSupportedTargetPlatformsForCurrentProject() = 0;

	/** Called when the target platforms for the current project are changed */
	DECLARE_MULTICAST_DELEGATE(FOnTargetPlatformsForCurrentProjectChangedEvent);
	virtual FOnTargetPlatformsForCurrentProjectChangedEvent& OnTargetPlatformsForCurrentProjectChanged() = 0;

	/**
	 * Checks whether the current project has default settings for plugins (ie. does not enable any new plugins, or disable any default plugins)
	 * 
	 * @return	True if the project has the default plugin settings.
	 */
	virtual bool HasDefaultPluginSettings() const = 0;

	/**
	 * Sets whether a plugin is enabled for the current project configuration.
	 * Potentially updates the current project descriptor, but does not save to disk and may require restarting to load it.
	 * @note Use IsCurrentProjectDirty() to tell whether the project was actually modified.
	 * 
	 * @param	PluginName		Name of the plugin
	 * @param	bEnabled		Whether to enable or disable the plugin
	 * @param	OutFailReason	On failure, gives an error message
	 * @return	False on failure to update the current project descriptor.
	 */
	virtual bool SetPluginEnabled(const FString& PluginName, bool bEnabled, FText& OutFailReason) = 0;

	/**
	 * 
	 */
	virtual bool RemovePluginReference(const FString& PluginName, FText& OutFailReason) = 0;

	/**
	 * Updates a directory to be scanned for plugins (added or removed)
	 *
	 * @param Dir the directory to scan
	 * @param bAddOrRemove whether to add or remove the directory
	 * @return Whether the plugin directory list was changed.
	 */
	virtual bool UpdateAdditionalPluginDirectory(const FString& Dir, const bool bAddOrRemove) = 0;

	/** Returns the list of additional directories to be scanned for plugins (aside from the engine and project plugin directories). */
	virtual const TArray<FString>& GetAdditionalPluginDirectories() const = 0;

	/**
	 * Checks whether the current loaded project has been modified but not saved to disk
	 */
	virtual bool IsCurrentProjectDirty() const = 0;

	/**
	 * Saves the current project to the project path
	 *
	 * @param	OutFailReason	On failure, gives an error message
	 * @return	True if the project was saved successfully
	 */
	virtual bool SaveCurrentProjectToDisk(FText& OutFailReason) = 0;

	/**
	 * Gets the enterprise flag value on the current project
	 *
	 * @return	True if the project is an Enterprise project
	 */
	virtual bool IsEnterpriseProject() = 0;

	/**
	 * Sets the enterprise flag value on the current project
	 *
	 * @param	bValue	The value to set the enterprise flag to
	 */
	virtual void SetIsEnterpriseProject(bool bValue) = 0;
	
	/**
	 * Access array used to cache current project's list of module context infos
	 */
	virtual TArray<FModuleContextInfo>& GetCurrentProjectModuleContextInfos() = 0;

	/** Returns true if project file write should be suppressed. */
	virtual bool IsSuppressingProjectFileWrite() const = 0;

	/** Suppress project file writes. */
	virtual void AddSuppressProjectFileWrite(const FName InName) = 0;

	/** Removes suppression of project file writes. */
	virtual void RemoveSuppressProjectFileWrite(const FName InName) = 0;
};
