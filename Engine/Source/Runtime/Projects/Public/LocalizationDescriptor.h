// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Serialization/JsonWriter.h"

class FJsonObject;
class FText;

/**
 * Policy by which the localization data associated with a target should be loaded.
 */
namespace ELocalizationTargetDescriptorLoadingPolicy
{
	enum Type
	{
		/** The localization data will never be loaded automatically. */
		Never,
		/** The localization data will always be loaded automatically. */
		Always,
		/** The localization data will only be loaded when running the editor. Use if the target localizes the editor. */
		Editor,
		/** The localization data will only be loaded when running the game. Use if the target localizes your game. */
		Game,
		/** The localization data will only be loaded if the editor is displaying localized property names. */
		PropertyNames,
		/** The localization data will only be loaded if the editor is displaying localized tool tips. */
		ToolTips,
		/** NOTE: If you add a new value, make sure to update the ToString() method below!. */
		Max,
	};

	/**
	 * Converts a string to a ELocalizationTargetDescriptorLoadingPolicy::Type value
	 *
	 * @param	The string to convert to a value
	 * @return	The corresponding value, or 'Max' if the string is not valid.
	 */
	PROJECTS_API ELocalizationTargetDescriptorLoadingPolicy::Type FromString(const TCHAR *Text);

	/**
	 * Returns the name of a localization loading policy.
	 *
	 * @param	The value to convert to a string
	 * @return	The string representation of this enum value
	 */
	PROJECTS_API const TCHAR* ToString(const ELocalizationTargetDescriptorLoadingPolicy::Type Value);
};

/** How the localization target's localization config files are generated during the localization gather pipeline.*/
namespace ELocalizationConfigGenerationPolicy
{
	enum Type
	{
		/** This localization target does not have localization config files associated with it and no localization content files will be generated for it during the localizaiton pipeline.*/
		Never,
		/** The user has provided localization config files for this localization target and they will be used to generate the localization content files for the localization target.*/
		User,
		/** Temporary localization config files will be generated for the localization target during the localization pipeline to generate the localization content files. After the localization pipeline is complete, the temporary files will be deleted.*/
		Auto,
		/** NOTE: If you add a new value, make sure to update the ToString() method below!. */
		Max,
	};

	/**
	 * Converts a string to a ELocalizationConfigGenerationPolicy::Type value
	 *
	 * @param	The string to convert to a value
	 * @return	The corresponding value, or 'Max' if the string is not valid.
	 */
	PROJECTS_API ELocalizationConfigGenerationPolicy::Type FromString(const TCHAR* Text);

	/**
	 * Returns the name of a ELocalizationConfigGenerationPolicy::Type
	 *
	 * @param	The value to convert to a string
	 * @return	The string representation of this enum value
	 */
	PROJECTS_API const TCHAR* ToString(const ELocalizationConfigGenerationPolicy::Type Value);
};


/**
 * Description of a localization target.
 */
struct FLocalizationTargetDescriptor
{
	/** Name of this target */
	FString Name;

	/** When should the localization data associated with a target should be loaded? */
	ELocalizationTargetDescriptorLoadingPolicy::Type LoadingPolicy;

	/** How the localizationc config files associated with the localization target are generated */
	ELocalizationConfigGenerationPolicy::Type ConfigGenerationPolicy;

	/** Normal constructor */
	PROJECTS_API FLocalizationTargetDescriptor(FString InName = FString(), ELocalizationTargetDescriptorLoadingPolicy::Type InLoadingPolicy = ELocalizationTargetDescriptorLoadingPolicy::Never, ELocalizationConfigGenerationPolicy::Type InGenerationPolicy = ELocalizationConfigGenerationPolicy::Never);

	/** Reads a descriptor from the given JSON object */
	PROJECTS_API bool Read(const FJsonObject& InObject, FText* OutFailReason = nullptr);

	/** Reads a descriptor from the given JSON object */
	PROJECTS_API bool Read(const FJsonObject& InObject, FText& OutFailReason);

	/** Reads an array of targets from the given JSON object */
	static PROJECTS_API bool ReadArray(const FJsonObject& InObject, const TCHAR* InName, TArray<FLocalizationTargetDescriptor>& OutTargets, FText* OutFailReason = nullptr);

	/** Reads an array of targets from the given JSON object */
	static PROJECTS_API bool ReadArray(const FJsonObject& InObject, const TCHAR* InName, TArray<FLocalizationTargetDescriptor>& OutTargets, FText& OutFailReason);

	/** Writes a descriptor to JSON */
	PROJECTS_API void Write(TJsonWriter<>& Writer) const;

	/** Updates the given json object with values in this descriptor */
	PROJECTS_API void UpdateJson(FJsonObject& JsonObject) const;

	/** Writes an array of targets to JSON */
	static PROJECTS_API void WriteArray(TJsonWriter<>& Writer, const TCHAR* ArrayName, const TArray<FLocalizationTargetDescriptor>& Descriptors);

	/** Updates an array of descriptors in the specified JSON field (indexed by name) */
	static PROJECTS_API void UpdateArray(FJsonObject& JsonObject, const TCHAR* ArrayName, const TArray<FLocalizationTargetDescriptor>& Descriptors);

	/** Returns true if we should load this localization target based upon the current runtime environment */
	PROJECTS_API bool ShouldLoadLocalizationTarget() const;
};
