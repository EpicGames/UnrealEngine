// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "Serialization/JsonWriter.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FJsonObject;
class FText;

enum class EModuleLoadResult;
enum class EModuleUnloadResult;

/**
 * Phase at which this module should be loaded during startup.
 */
namespace ELoadingPhase
{
	enum Type
	{
		/** As soon as possible - in other words, uplugin files are loadable from a pak file (as well as right after PlatformFile is set up in case pak files aren't used) Used for plugins needed to read files (compression formats, etc) */
		EarliestPossible,

		/** Loaded before the engine is fully initialized, immediately after the config system has been initialized.  Necessary only for very low-level hooks */
		PostConfigInit,

		/** The first screen to be rendered after system splash screen */
		PostSplashScreen,

		/** Loaded before coreUObject for setting up manual loading screens, used for our chunk patching system */
		PreEarlyLoadingScreen,

		/** Loaded before the engine is fully initialized for modules that need to hook into the loading screen before it triggers */
		PreLoadingScreen,

		/** Right before the default phase */
		PreDefault,

		/** Loaded at the default loading point during startup (during engine init, after game modules are loaded.) */
		Default,

		/** Right after the default phase */
		PostDefault,

		/** After the engine has been initialized */
		PostEngineInit,

		/** Do not automatically load this module */
		None,

		// NOTE: If you add a new value, make sure to update the ToString() method below!
		Max
	};

	/**
	 * Converts a string to a ELoadingPhase::Type value
	 *
	 * @param	The string to convert to a value
	 * @return	The corresponding value, or 'Max' if the string is not valid.
	 */
	PROJECTS_API ELoadingPhase::Type FromString( const TCHAR *Text );

	/**
	 * Returns the name of a module load phase.
	 *
	 * @param	The value to convert to a string
	 * @return	The string representation of this enum value
	 */
	PROJECTS_API const TCHAR* ToString( const ELoadingPhase::Type Value );
};

/**
 * Environment that can load a module.
 */
namespace EHostType
{
	enum Type : int
	{
		// Loads on all targets, except programs.
		Runtime,
		
		// Loads on all targets, except programs and the editor running commandlets.
		RuntimeNoCommandlet,
		
		// Loads on all targets, including supported programs.
		RuntimeAndProgram,
		
		// Loads only in cooked games.
		CookedOnly,

		// Only loads in uncooked games.
		UncookedOnly,

		// Deprecated due to ambiguities. Only loads in editor and program targets, but loads in any editor mode (eg. -game, -server).
		// Use UncookedOnly for the same behavior (eg. for editor blueprint nodes needed in uncooked games), or DeveloperTool for modules
		// that can also be loaded in cooked games but should not be shipped (eg. debugging utilities).
		Developer,

		// Loads on any targets where bBuildDeveloperTools is enabled.
		DeveloperTool,

		// Loads only when the editor is starting up.
		Editor,
		
		// Loads only when the editor is starting up, but not in commandlet mode.
		EditorNoCommandlet,

		// Loads only on editor and program targets
		EditorAndProgram,

		// Only loads on program targets.
		Program,
		
		// Loads on all targets except dedicated clients.
		ServerOnly,
		
		// Loads on all targets except dedicated servers.
		ClientOnly,

		// Loads in editor and client but not in commandlets.
		ClientOnlyNoCommandlet,
		
		//~ NOTE: If you add a new value, make sure to update the ToString() method below!
		Max
	};

	/**
	 * Converts a string to a EHostType::Type value
	 *
	 * @param	The string to convert to a value
	 * @return	The corresponding value, or 'Max' if the string is not valid.
	 */
	PROJECTS_API EHostType::Type FromString( const TCHAR *Text );

	/**
	 * Converts an EHostType::Type value to a string literal
	 *
	 * @param	The value to convert to a string
	 * @return	The string representation of this enum value
	 */
	PROJECTS_API const TCHAR* ToString( const EHostType::Type Value );
};

/**
 * Description of a loadable module.
 */
struct FModuleDescriptor
{
	/** Name of this module */
	FName Name;

	/** Usage type of module */
	EHostType::Type Type;

	/** When should the module be loaded during the startup sequence?  This is sort of an advanced setting. */
	ELoadingPhase::Type LoadingPhase;

	/** List of allowed platforms */
	TArray<FString> PlatformAllowList;

	/** List of disallowed platforms */
	TArray<FString> PlatformDenyList;

	/** List of allowed platform architectures */
	TMap<FString,TArray<FString>> PlatformArchitectureAllowList;

	/** List of disallowed platform architectures */
	TMap<FString,TArray<FString>> PlatformArchitectureDenyList;

	/** List of allowed targets */
	TArray<EBuildTargetType> TargetAllowList;

	/** List of disallowed targets */
	TArray<EBuildTargetType> TargetDenyList;

	/** List of allowed target configurations */
	TArray<EBuildConfiguration> TargetConfigurationAllowList;

	/** List of disallowed target configurations */
	TArray<EBuildConfiguration> TargetConfigurationDenyList;

	/** List of allowed programs */
	TArray<FString> ProgramAllowList;

	/** List of disallowed programs */
	TArray<FString> ProgramDenyList;

	/** List of allowed game targets */
	TArray<FString> GameTargetAllowList;

	/** List of disallowed game targets */
	TArray<FString> GameTargetDenyList;
	
	/** List of additional dependencies for building this module. */
	TArray<FString> AdditionalDependencies;

	/** When true, an empty PlatformAllowList is interpeted as 'no platforms' with the expectation that explict platforms will be added in plugin extensions */
	bool bHasExplicitPlatforms;


	/** Normal constructor */
	PROJECTS_API FModuleDescriptor(const FName InName = NAME_None, EHostType::Type InType = EHostType::Runtime, ELoadingPhase::Type InLoadingPhase = ELoadingPhase::Default);

	/** Reads a descriptor from the given JSON object */
	PROJECTS_API bool Read(const FJsonObject& Object, FText* OutFailReason = nullptr);

	/** Reads a descriptor from the given JSON object */
	PROJECTS_API bool Read(const FJsonObject& Object, FText& OutFailReason);

	/** Reads an array of modules from the given JSON object */
	static PROJECTS_API bool ReadArray(const FJsonObject& Object, const TCHAR* Name, TArray<FModuleDescriptor>& OutModules, FText* OutFailReason = nullptr);

	/** Reads an array of modules from the given JSON object */
	static PROJECTS_API bool ReadArray(const FJsonObject& Object, const TCHAR* Name, TArray<FModuleDescriptor>& OutModules, FText& OutFailReason);

	/** Writes a descriptor to JSON */
	PROJECTS_API void Write(TJsonWriter<>& Writer) const;

	/** Updates the given json object with values in this descriptor */
	PROJECTS_API void UpdateJson(FJsonObject& JsonObject) const;

	/** Writes an array of modules to JSON */
	static PROJECTS_API void WriteArray(TJsonWriter<>& Writer, const TCHAR* ArrayName, const TArray<FModuleDescriptor>& Modules);

	/** Updates an array of module descriptors in the specified JSON field (indexed by module name) */
	static PROJECTS_API void UpdateArray(FJsonObject& JsonObject, const TCHAR* ArrayName, const TArray<FModuleDescriptor>& Modules);

	/** Tests whether the module should be built for the given target */
	PROJECTS_API bool IsCompiledInConfiguration(const FString& Platform, EBuildConfiguration Configuration, const FString& TargetName, EBuildTargetType TargetType, bool bBuildDeveloperTools, bool bBuildRequiresCookedData, const FString& Architecture = FString()) const;

	/** Tests whether the module should be built for the current engine configuration */
	PROJECTS_API bool IsCompiledInCurrentConfiguration() const;

	/** Tests whether the module should be loaded for the current engine configuration */
	PROJECTS_API bool IsLoadedInCurrentConfiguration() const;

	/** Loads all the modules for a given loading phase. Returns a map of module names to load errors */
	static PROJECTS_API void LoadModulesForPhase(ELoadingPhase::Type LoadingPhase, const TArray<FModuleDescriptor>& Modules, TMap<FName, EModuleLoadResult>& ModuleLoadErrors);

	/** Unloads all the modules for a given loading phase. Returns a map of module names to load errors. bSkipUnload can be used to simulate unloading */
	static PROJECTS_API void UnloadModulesForPhase(ELoadingPhase::Type LoadingPhase, const TArray<FModuleDescriptor>& Modules, TMap<FName, EModuleUnloadResult>& OutErrors, bool bSkipUnload = false, bool bAllowUnloadCode = true);

#if !IS_MONOLITHIC
	/** Checks that all modules are compatible with the current engine version. Returns false and appends a list of names to OutIncompatibleFiles if not. */
	static PROJECTS_API bool CheckModuleCompatibility(const TArray<FModuleDescriptor>& Modules, TArray<FString>& OutIncompatibleFiles);
#endif
};

/** Context information used when validating that source code is being placed in the correct place for a given module */
struct FModuleContextInfo
{
	/** Path to the Source folder of the module */
	FString ModuleSourcePath;

	/** Name of the module */
	FString ModuleName;

	/** Type of this module, eg, Runtime, Editor, etc */
	EHostType::Type ModuleType;
};
