// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CustomBuildSteps.h"
#include "HAL/Platform.h"
#include "LocalizationDescriptor.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Optional.h"
#include "ModuleDescriptor.h"
#include "PluginDisallowedDescriptor.h"
#include "PluginReferenceDescriptor.h"
#include "Serialization/JsonWriter.h"
#include "Templates/SharedPointer.h"
#include "VerseScope.h"

class FJsonObject;
class FJsonValue;
class FText;

/**
 * Setting for whether a plugin is enabled by default
 */ 
enum class EPluginEnabledByDefault : uint8
{
	Unspecified,
	Enabled,
	Disabled,
};

/**
 * Descriptor for plugins. Contains all the information contained within a .uplugin file.
 */
struct FPluginDescriptor
{
	/** Version number for the plugin.  The version number must increase with every version of the plugin, so that the system 
	    can determine whether one version of a plugin is newer than another, or to enforce other requirements.  This version
		number is not displayed in front-facing UI.  Use the VersionName for that. */
	int32 Version;

	/** Name of the version for this plugin.  This is the front-facing part of the version number.  It doesn't need to match
	    the version number numerically, but should be updated when the version number is increased accordingly. */
	FString VersionName;

	/** Friendly name of the plugin */
	FString FriendlyName;

	/** Description of the plugin */
	FString Description;

	/** The name of the category this plugin */
	FString Category;

	/** The company or individual who created this plugin.  This is an optional field that may be displayed in the user interface. */
	FString CreatedBy;

	/** Hyperlink URL string for the company or individual who created this plugin.  This is optional. */
	FString CreatedByURL;

	/** Documentation URL string. */
	FString DocsURL;

	/** Marketplace URL for this plugin. This URL will be embedded into projects that enable this plugin, so we can redirect to the marketplace if a user doesn't have it installed. */
	FString MarketplaceURL;

	/** Support URL/email for this plugin. */
	FString SupportURL;

	/** Version of the engine that this plugin is compatible with */
	FString EngineVersion;

	/** Optional custom virtual path to display in editor to better organize. Inserted just before this plugin's directory in the path: /All/Plugins/EditorCustomVirtualPath/PluginName */
	FString EditorCustomVirtualPath;

	/** Controls a subset of platforms that can use this plugin, and which ones will stage the .uplugin file and content files. 
	Generally, for code plugins, it should be the union of platforms that the modules in the plugin are compiled for. */
	TArray<FString> SupportedTargetPlatforms;

	/** List of programs that are supported by this plugin. */
	TArray<FString> SupportedPrograms;

	/** If specified, this is the real plugin that this one is just extending */
	FString ParentPluginName;

	/** List of all modules associated with this plugin */
	TArray<FModuleDescriptor> Modules;

	/** List of all localization targets associated with this plugin */
	TArray<FLocalizationTargetDescriptor> LocalizationTargets;

	/** The Verse path to the root of this plugin's content directory */
	FString VersePath;

	/** Set when a plugin is loaded that has the DeprecatedEngineVersion set. */
	FString DeprecatedEngineVersion; 

	/** The version of the Verse language that this plugin targets.
		If no value is specified, the latest stable version is used. */
	TOptional<uint32> VerseVersion;

	/** Origin/visibility of Verse code in this plugin's Content/Verse folder */
	EVerseScope::Type VerseScope = EVerseScope::PublicUser;

	/** Whether this plugin has SceneGraph enabled, which impacts the generated Verse Asset Digest. */
	bool bEnableSceneGraph = false;

	/** Whether this plugin has SceneGraph Animation enabled, which impacts the generated Verse Asset Digest. */
	bool bEnableSceneGraphAnimation = false;

	/** If to generate Verse source code definitions from assets contained in this plugin */
	bool bEnableVerseAssetReflection = false;

	/** Use IAD for this plugin */
	bool bEnableIAD = false;

	/** Use IAS for this plugin. This is likely to be removed in the near future. */
	bool bEnableIAS = true;

	/** Whether this plugin should be enabled by default for all projects */
	EPluginEnabledByDefault EnabledByDefault;

	/** Can this plugin contain content? */
	bool bCanContainContent;

	/** Can this plugin contain Verse code? */
	bool bCanContainVerse;

	/** Marks the plugin as beta in the UI */
	bool bIsBetaVersion;

	/** Marks the plugin as experimental in the UI */
	bool bIsExperimentalVersion;

	/** Signifies that the plugin was installed on top of the engine */
	bool bInstalled;

	/** For plugins that are under a platform folder (eg. /PS4/), determines whether compiling the plugin requires the build platform and/or SDK to be available */
	bool bRequiresBuildPlatform;

	/** For auto-generated plugins that should not be listed in the plugin browser for users to disable freely. */
	bool bIsHidden;

	/** Prevents other plugins from depending on this plugin. */
	bool bIsSealed;

	/** Prevents this plugin from containing code or modules. */
	bool bNoCode;

	/** When true, this plugin's modules will not be loaded automatically nor will it's content be mounted automatically. It will load/mount when explicitly requested and LoadingPhases will be ignored */
	bool bExplicitlyLoaded;

	/** When true, an empty SupportedTargetPlatforms is interpreted as 'no platforms' with the expectation that explicit platforms will be added in plugin platform extensions */
	bool bHasExplicitPlatforms;

	/** If true, this plugin from a platform extension extending another plugin */
	bool bIsPluginExtension;

	/** If true, the GameFeaturePlugin this is on should have no content other than the GameFeatureData asset. */
	bool bNoGameFeatureContent = false;

#if WITH_EDITOR || IS_PROGRAM
	/** Pre-build steps for each host platform */
	FCustomBuildSteps PreBuildSteps;

	/** Post-build steps for each host platform */
	FCustomBuildSteps PostBuildSteps;
#endif

	/** Plugins used by this plugin */
	TArray<FPluginReferenceDescriptor> Plugins;

	/** Plugins that cannot be used by this plugin */
	TArray<FPluginDisallowedDescriptor> DisallowedPlugins;


#if WITH_EDITOR
	/** Cached json for custom data */
	mutable TSharedPtr<FJsonObject> CachedJson;

	/** Additional fields to write */
	TMap<FString, TSharedPtr<FJsonValue>> AdditionalFieldsToWrite;

	/** Additional fields to remove when updating the Json */
	TSet<FString> AdditionalFieldsToRemove;
#endif

	/** Return the .uplugin extension (with dot) */
	static PROJECTS_API const FString& GetFileExtension();

	/** Constructor. */
	PROJECTS_API FPluginDescriptor();

	/** Loads the descriptor from the given file. */
	PROJECTS_API bool Load(const TCHAR* FileName, FText* OutFailReason = nullptr);

	/** Loads the descriptor from the given file. */
	PROJECTS_API bool Load(const FString& FileName, FText* OutFailReason = nullptr);

	/** Loads the descriptor from the given file. */
	PROJECTS_API bool Load(const FString& FileName, FText& OutFailReason);

	/* Reads the descriptor using the global CustomPluginDescriptorReaderDelegate */
	PROJECTS_API bool ReadWithCustomPluginDescriptorReader(const TCHAR* FileName, FText* OutFailReason = nullptr);

	/** Reads the descriptor from the given string */
	PROJECTS_API bool Read(const FString& Text, FText* OutFailReason = nullptr);

	/** Reads the descriptor from the given string */
	PROJECTS_API bool Read(const FString& Text, FText& OutFailReason);

	/** Reads the descriptor from the given JSON object */
	PROJECTS_API bool Read(const FJsonObject& Object, FText* OutFailReason = nullptr);

	/** Reads the descriptor from the given JSON object */
	PROJECTS_API bool Read(const FJsonObject& Object, FText& OutFailReason);

	/** Saves the descriptor to the given file. */
	PROJECTS_API bool Save(const TCHAR* FileName, FText* OutFailReason = nullptr) const;

	/** Saves the descriptor to the given file. */
	PROJECTS_API bool Save(const FString& FileName, FText* OutFailReason = nullptr) const;

	/** Saves the descriptor to the given file. */
	PROJECTS_API bool Save(const FString& FileName, FText& OutFailReason) const;

	/** Writes a descriptor to JSON */
	PROJECTS_API void Write(FString& Text) const;

	/** Writes a descriptor to JSON */
	PROJECTS_API void Write(TJsonWriter<>& Writer) const;

	/** Updates the given json object with values in this descriptor */
	PROJECTS_API void UpdateJson(FJsonObject& JsonObject) const;

	/**
	 * Updates the content of the specified plugin file with values in this descriptor
	 * (hence preserving json fields that the plugin descriptor doesn't know about)
	 */
	PROJECTS_API bool UpdatePluginFile(const FString& FileName, FText* OutFailReason = nullptr) const;

	/**
	 * Updates the content of the specified plugin file with values in this descriptor
	 * (hence preserving json fields that the plugin descriptor doesn't know about)
	 */
	PROJECTS_API bool UpdatePluginFile(const FString& FileName, FText& OutFailReason) const;

	/** Determines whether the plugin supports the given platform */
	PROJECTS_API bool SupportsTargetPlatform(const FString& Platform) const;

	// Custom hook for providing a plugin descriptor (as an FJsonObject) based on a filename
	DECLARE_DELEGATE_RetVal_FourParams(bool, FPluginDescriptorReaderDelegate, const TCHAR*/*Filename*/, FText*/*OutFailReason*/,  TSharedPtr<FJsonObject>&/*OutJson*/, bool&/*OutRetryWithNative*/);
	PROJECTS_API static FPluginDescriptorReaderDelegate CustomPluginDescriptorReaderDelegate;
};

namespace UE::PluginDescriptor::Internal
{
	PROJECTS_API bool ReadFile(const TCHAR* FileName, FString& Text, FText* OutFailReason = nullptr);
	PROJECTS_API TSharedPtr<FJsonObject> DeserializeJson(const FString& Text, FText* OutFailReason = nullptr);
}
