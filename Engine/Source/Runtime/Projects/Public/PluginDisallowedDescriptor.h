// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Serialization/JsonWriter.h"

class FJsonObject;
class FText;

struct FPluginDisallowedDescriptor
{
	FString Name;

#if WITH_EDITOR
	FString Comment;
#endif

	/** Reads the descriptor from the given JSON object */
	PROJECTS_API bool Read(const FJsonObject& Object, FText* OutFailReason = nullptr);

	/** Writes a descriptor to JSON */
	PROJECTS_API void Write(TJsonWriter<>& Writer) const;

	/** Updates the given json object with values in this descriptor */
	PROJECTS_API void UpdateJson(FJsonObject& JsonObject) const;

	static PROJECTS_API bool ReadArray(const FJsonObject& Object, const TCHAR* Name, TArray<FPluginDisallowedDescriptor>& OutPlugins, FText* OutFailReason = nullptr);

	/** Writes an array of plugin references to JSON */
	static PROJECTS_API void WriteArray(TJsonWriter<>& Writer, const TCHAR* ArrayName, const TArray<FPluginDisallowedDescriptor>& Plugins);

	/** Updates an array of plugin references in the specified JSON field (indexed by plugin name) */
	static PROJECTS_API void UpdateArray(FJsonObject& JsonObject, const TCHAR* ArrayName, const TArray<FPluginDisallowedDescriptor>& Plugins);
};