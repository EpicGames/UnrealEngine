// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "PluginDescriptor.h"

class FJsonObject;
class FText;

/**
 * Descriptor for plugins. Contains all the information contained within a .uplugin file.
 */
struct FPluginManifestEntry
{
	/** Normalized path to the plugin file */
	FString File;

	/** The plugin descriptor */
	FPluginDescriptor Descriptor;
};

/**
 * Manifest of plugins. Descriptor for plugins. Contains all the information contained within a .uplugin file.
 */
struct FPluginManifest
{
	/** List of plugins in this manifest */
	TArray<FPluginManifestEntry> Contents;

	/** Loads the descriptor from the given file. */
	PROJECTS_API bool Load(const FString& FileName, FText& OutFailReason);

	/** Reads the descriptor from the given JSON object */
	PROJECTS_API bool Read(const FJsonObject& Object, FText& OutFailReason);
};
