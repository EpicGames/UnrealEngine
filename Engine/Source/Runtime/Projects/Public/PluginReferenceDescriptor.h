// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "CustomBuildSteps.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "LocalizationDescriptor.h"
#include "Misc/Optional.h"
#include "ModuleDescriptor.h"
#include "Serialization/JsonWriter.h"
#include "Templates/SharedPointer.h"

class FJsonObject;
class FJsonValue;
class FText;

/**
 * Descriptor for a plugin reference. Contains the information required to enable or disable a plugin for a given platform.
 */
struct FPluginReferenceDescriptor
{
	/** Name of the plugin */
	FString Name;

	/** Whether it should be enabled by default */
	bool bEnabled;

	/** Whether this plugin is optional, and the game should silently ignore it not being present */
	bool bOptional;

	/** Whether this plugin dependency should also activate when the parent plugin is activated by the GameFeaturePlugin system. */
	bool bActivate;

	/** When true, empty SupportedTargetPlatforms and PlatformAllowList are interpreted as 'no platforms' with the expectation that explicit platforms will be added in plugin platform extensions */
	bool bHasExplicitPlatforms;
	
	/** Description of the plugin for users that do not have it installed. */
	FString Description;

	/** URL for this plugin on the marketplace, if the user doesn't have it installed. */
	FString MarketplaceURL;

	/** If enabled, list of platforms for which the plugin should be enabled (or all platforms if blank). */
	TArray<FString> PlatformAllowList;

	/** If enabled, list of platforms for which the plugin should be disabled. */
	TArray<FString> PlatformDenyList;
 
	/** If enabled, list of target configurations for which the plugin should be enabled (or all target configurations if blank). */
	TArray<EBuildConfiguration> TargetConfigurationAllowList;

	/** If enabled, list of target configurations for which the plugin should be disabled. */
	TArray<EBuildConfiguration> TargetConfigurationDenyList;

	/** If enabled, list of targets for which the plugin should be enabled (or all targets if blank). */
	TArray<EBuildTargetType> TargetAllowList;

	/** If enabled, list of targets for which the plugin should be disabled. */
	TArray<EBuildTargetType> TargetDenyList;

	/** Overrides the loading phase in uplugin so that a plugin can be explicitly loaded for specified targets. */
	TArray<EBuildTargetType> TargetsToExplicitlyLoad;

	/** The list of supported target platforms for this plugin. This field is copied from the plugin descriptor, and supplements the user's allowed/denied platforms. */
	TArray<FString> SupportedTargetPlatforms;

	/** When set, specifies a specific version of the plugin that this references. */
	TOptional<int32> RequestedVersion;

#if WITH_EDITOR
	/** Cached json for custom data */
	mutable TSharedPtr<FJsonObject> CachedJson;

	/** Additional fields to write */
	TMap<FString, TSharedPtr<FJsonValue>> AdditionalFieldsToWrite;
#endif //if WITH_EDITOR

	/** Constructor */
	PROJECTS_API FPluginReferenceDescriptor(FString InName = TEXT(""), bool bInEnabled = false);

	/** Determines whether the plugin is enabled for the given platform */
	PROJECTS_API bool IsEnabledForPlatform(const FString& Platform) const;

	/** Determines whether the plugin is enabled for the given target configuration */
	PROJECTS_API bool IsEnabledForTargetConfiguration(EBuildConfiguration Configuration) const;

	/** Determines whether the plugin is enabled for the given target */
	PROJECTS_API bool IsEnabledForTarget(EBuildTargetType TargetType) const;

	/** Determines if the referenced plugin is supported for the given platform */
	PROJECTS_API bool IsSupportedTargetPlatform(const FString& Platform) const;

	/** Reads the descriptor from the given JSON object */
	PROJECTS_API bool Read(const TSharedRef<FJsonObject>& Object, FText* OutFailReason = nullptr);

	UE_DEPRECATED(5.1, "Use Read(const TSharedRef<FJsonObject>&) instead.")
	PROJECTS_API bool Read(const FJsonObject& Object, FText* OutFailReason = nullptr, TSharedPtr<FJsonObject> ObjectPtr = nullptr);

	UE_DEPRECATED(5.1, "Use Read(const TSharedRef<FJsonObject>&) instead.")
	PROJECTS_API bool Read(const FJsonObject& Object, FText& OutFailReason, TSharedPtr<FJsonObject> ObjectPtr = nullptr);

	/** Reads an array of modules from the given JSON object */
	static PROJECTS_API bool ReadArray(const FJsonObject& Object, const TCHAR* Name, TArray<FPluginReferenceDescriptor>& OutModules, FText* OutFailReason = nullptr);

	/** Reads an array of modules from the given JSON object */
	static PROJECTS_API bool ReadArray(const FJsonObject& Object, const TCHAR* Name, TArray<FPluginReferenceDescriptor>& OutModules, FText& OutFailReason);

	/** Writes a descriptor to JSON */
	PROJECTS_API void Write(TJsonWriter<>& Writer) const;
	
	/** Updates the given json object with values in this descriptor */
	PROJECTS_API void UpdateJson(FJsonObject& JsonObject) const;

	/** Writes an array of plugin references to JSON */
	static PROJECTS_API void WriteArray(TJsonWriter<>& Writer, const TCHAR* ArrayName, const TArray<FPluginReferenceDescriptor>& Plugins);

	/** Updates an array of plugin references in the specified JSON field (indexed by plugin name) */
	static PROJECTS_API void UpdateArray(FJsonObject& JsonObject, const TCHAR* ArrayName, const TArray<FPluginReferenceDescriptor>& Plugins);

#if WITH_EDITOR
	/** Gets the string value for a given key by first looking into AdditionalFieldsToWrite and then CachedJson */
	PROJECTS_API bool GetAdditionalStringField(const FString& Key, FString& OutValue) const;
#endif //if WITH_EDITOR
};
