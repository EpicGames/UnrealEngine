// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CustomBuildSteps.h"
#include "HAL/Platform.h"
#include "ModuleDescriptor.h"
#include "PluginReferenceDescriptor.h"
#include "Serialization/JsonWriter.h"
#include "UObject/NameTypes.h"
#include "JsonUtils/RapidJsonUtils.h"

class FJsonObject;
class FText;

/**
 * Version numbers for project descriptors.
 */ 
namespace EProjectDescriptorVersion
{
	enum Type
	{
		Invalid = 0,
		Initial = 1,
		NameHash = 2,
		ProjectPluginUnification = 3,
		// !!!!!!!!!! IMPORTANT: Remember to also update LatestPluginDescriptorFileVersion in Plugins.cs (and Plugin system documentation) when this changes!!!!!!!!!!!
		// -----<new versions can be added before this line>-------------------------------------------------
		// - this needs to be the last line (see note below)
		LatestPlusOne,
		Latest = LatestPlusOne - 1
	};
}

/**
 * Descriptor for projects. Contains all the information contained within a .uproject file.
 */
struct FProjectDescriptor
{
	/** Descriptor version number. */
	EProjectDescriptorVersion::Type FileVersion;

	/** 
	 * The engine to open this project with. Set this value using IDesktopPlatform::SetEngineIdentifierForProject to ensure that
	 * the most portable value for this field is used.
	 *
	 * This field allows us to open the right version of the engine when you double-click on a .uproject file, and to detect when you 
	 * open a project with a different version of the editor and need the upgrade/downgrade UI flow. The normal engine 
	 * version doesn't work for those purposes, because you can have multiple 4.x branches in various states on one machine.
	 *
	 * For Launcher users, this field gets set to something stable like "4.7" or "4.8", so you can swap projects and game binaries 
	 * between users, and it'll automatically work on any platform or machine regardless of where the engine is installed. You 
	 * can only have one binary release of each major engine version installed at once.
	 * 
	 * For Perforce or Git users that branch the engine along with their games, this field is left blank. You can sync the repository 
	 * down on any platform and machine, and it can figure out which engine a project should use by looking up the directory 
	 * hierarchy until it finds one.
	 * 
	 * For other cases, where you have a source build of the engine but are working with a foreign project, we use a random identifier 
	 * for each local engine installation and use the registry to map it back to the engine directory. All bets are off as to which
	 * engine you should use to open it on a different machine, and using a random GUID ensures that every new machine triggers the
	 * engine selection UI when you open or attempt to generate project files for it.
	 * 
	 * For users which mount the engine through a Git submodule (where the engine is in a subdirectory of the project), this field 
	 * can be manually edited to be a relative path.
	 *
	 * @see IDesktopPlatform::GetEngineIdentifierForProject
	 * @see IDesktopPlatform::SetEngineIdentifierForProject
	 * @see IDesktopPlatform::GetEngineRootDirFromIdentifier
	 * @see IDesktopPlatform::GetEngineIdentifierFromRootDir
	 */
	FString EngineAssociation;

	/** Category to show under the project browser */
	FString Category;

	/** Description to show in the project browser */
	FString Description;

	/** List of all modules associated with this project */
	TArray<FModuleDescriptor> Modules;

	/** List of plugins for this project (may be enabled/disabled) */
	TArray<FPluginReferenceDescriptor> Plugins;

	/** Array of platforms that this project is targeting */
	TArray<FName> TargetPlatforms;

	/** A hash that is used to determine if the project was forked from a sample */
	uint32 EpicSampleNameHash;

	/** Custom steps to execute before building targets in this project */
	FCustomBuildSteps PreBuildSteps;

	/** Custom steps to execute after building targets in this project */
	FCustomBuildSteps PostBuildSteps;

	/** Indicates if this project is an Enterprise project */
	bool bIsEnterpriseProject;

	/** Indicates that enabled by default engine plugins should not be enabled unless explicitly enabled by the project or target files. */
	bool bDisableEnginePluginsByDefault;

	/** Constructor. */
	PROJECTS_API FProjectDescriptor();

	/** Signs the project given for the given filename */
	PROJECTS_API void Sign(const FString& FilePath);

	/** Checks whether the descriptor is signed */
	PROJECTS_API bool IsSigned(const FString& FilePath) const;

	/** Finds the index of a plugin in the references array */
	PROJECTS_API int32 FindPluginReferenceIndex(const FString& PluginName) const;

	/** Updates the supported target platforms list */
	PROJECTS_API void UpdateSupportedTargetPlatforms(const FName& InPlatformName, bool bIsSupported);

	/** Loads the descriptor from the given file. */
	PROJECTS_API bool Load(const FString& FileName, FText& OutFailReason);

	/** Reads the descriptor from the given JSON object */
	PROJECTS_API bool Read(const FJsonObject& Object, const FString& PathToProject, FText& OutFailReason);

	/** Saves the descriptor to the given file. */
	PROJECTS_API bool Save(const FString& FileName, FText& OutFailReason);

	/** Writes the descriptor to the given JSON object */
	PROJECTS_API void Write(TJsonWriter<>& Writer, const FString& PathToProject) const;

	/** Returns the extension used for project descriptors (uproject) */
	static PROJECTS_API FString GetExtension();

	/** @return - Access to the additional plugin directories */
	const TArray<FString>& GetAdditionalPluginDirectories() const
	{
		return AdditionalPluginDirectories;
	}

	/**
	 * Adds a directory to the additional plugin directories list. 
	 *
	 * @param Dir - the new directory to add (must be a full path)
	 * @return whether the plugin directory list was changed
	 */
	PROJECTS_API bool AddPluginDirectory(const FString& Dir);
	/**
	 * Removes the directory from the list to scan
	 *
	 * @param Dir the directory to remove (must be a full path)
	 * @return whether the plugin directory list was changed
	 */
	PROJECTS_API bool RemovePluginDirectory(const FString& Dir);

	/** @return - Access to the additional root directories */
	const TArray<FString>& GetAdditionalRootDirectories() const
	{
		return AdditionalRootDirectories;
	}

	/**
	 * Adds a directory to the additional root directories list. 
	 *
	 * @param Dir - the new directory to add (must be a full path)
	 * @return whether the root directory list was changed
	 */
	PROJECTS_API bool AddRootDirectory(const FString& Dir);
	/**
	 * Removes the directory from the list to scan
	 *
	 * @param Dir the directory to remove (must be a full path)
	 * @return whether the root directory list was changed
	 */
	PROJECTS_API bool RemoveRootDirectory(const FString& Dir);

#if WITH_EDITOR
	/** Returns whether the project has a module of the given name */
	PROJECTS_API bool HasModule(FName ModuleName) const;
#endif //if WITH_EDITOR

private:
	/** @return the path relative to this project if possible */
	PROJECTS_API const FString MakePathRelativeToProject(const FString& Dir, const FString& PathToProject) const;

	/**
	 * List of additional directories to scan for plugins.
	 * Paths are in memory as absolute paths. Conversion to/from path relative happens during Save/Load
	 */
	TArray<FString> AdditionalPluginDirectories;

	/**
	 * List of additional root directories to scan for modules.
	 * Paths are in memory as absolute paths. Conversion to/from path relative happens during Save/Load
	 */
	TArray<FString> AdditionalRootDirectories;

#if WITH_EDITOR
	/** Set of project module names */
	mutable TSet<FName> ModuleNamesCache;
#endif //if WITH_EDITOR

	friend struct FProjectsLoadingPluginDescriptorTestsHelper;
	bool Read(const UE::Json::FConstObject Object, const FString& PathToProject, FText& OutFailReason);
};
