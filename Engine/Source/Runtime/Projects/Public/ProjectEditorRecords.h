// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dom/JsonObject.h"

/**
 * Struct specifying the structure of the records file, this is used to maintain
 * an association between recently opened projects and the location of the editor engine
 * used to open them. This is meant to be a machine wide record to facilitate locating 
 * the right engine executable.
 * 
 * Usage: 
 * - QueueUpdate to update the record
 * - Load to read from the record
 */
struct FProjectEditorRecord
{
	/** Property used to specify a Projects list */
	PROJECTS_API static const FString ProjectsProperty;

	/** Property used to specify a SubProjects list */
	PROJECTS_API static const FString SubProjectProperty;

	/** Property used to specify the path of the engine used to open the project */
	PROJECTS_API static const FString EngineLocationProperty;

	/** Property used to specify the path of the base dir of the engine used for the project */
	PROJECTS_API static const FString BaseDirProperty;
	
	/** Property used to specify a last accessed timestamp */
	PROJECTS_API static const FString TimestampProperty;

	/** Property used to specify an Epic App */
	PROJECTS_API static const FString EpicAppProperty;

	/** The json contents of the records loaded/saved to disk */
	TSharedPtr<FJsonObject> ProjectEditorJson;

	/**
	 * This is the main way to interact with the json contents to add entries in a safe way.
	 *
	 * QueueUpdate schedules a task that will try to run in another thread and perform these operations:
	 * - Acquires a system wide lock
	 * - Loads the latest records from disk
	 * - Runs the function passed as a parameter with the loaded contents
	 * - Saves the records to disk
	 * - Releases system wide lock
	 *
	 * @param Function used to update the records in a worker thread
	 */
	PROJECTS_API static void QueueUpdate(TUniqueFunction<void(FProjectEditorRecord&)>&& InUpdateFunction);

	/**
	 * Loads a project editor record from disk, using the default location %LOCALAPPDATA%/UnrealEngine/Editor/ProjectEditorRecords.json
	 * location can be overriden by cvar r.Editor.ProjectEditorRecordsFile
	 * 
	 * @return	The struct loaded from disk, or an empty one.
	 */
	PROJECTS_API static FProjectEditorRecord Load();
	
	/**
	 * Finds or adds an object property to the root of the json
	 * 
	 * @param The name of the property to find or add
	 * @return Shared pointer to the json object corresponding to the property input
	 */
	PROJECTS_API const TSharedPtr<FJsonObject> FindOrAddProperty(const FString& InProperty);

	/**
	 * Creates a JsonObject containing the default entries 
	 * - Path to the UnrealEditor.exe used when calling this function (meant to be UnrealEditor.exe)
	 * - Path to the base directory when calling this function
	 * - Current timestamp
	 * 
	 * @return JsonObject containing the default entries for a project
	 */
	PROJECTS_API static TSharedPtr<FJsonObject> MakeDefaultProperties();


	/**
	 * Waits for any queued async tasks to finish if there are any
	 */
	PROJECTS_API static void TearDown();

private:
	FProjectEditorRecord() = default;

	/** Handle for the latest async task */
	static FGraphEventRef AsyncUpdateTask;

	/** Number of days that have to pass for an entry to be considered stale and be removed on save */
	const int32 DaysToKeepRecords = 60;

	/** Goes over the json object and prunes anything older than DaysToKeepRecords */
	void PruneOldEntries(const TSharedPtr<FJsonObject>& JsonObject);

	/** Gets the location of the file on disk */
	static const FString GetFileLocation();

	/** Saves the contents of this structure to disk */
	bool Save();
};


