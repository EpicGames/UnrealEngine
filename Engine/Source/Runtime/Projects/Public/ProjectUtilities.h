// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::ProjectUtilities
{
	/**
	 * Allows a monolithic target to control the game directory by the presence of commandline arguments for the project.
	 */
	PROJECTS_API void ParseProjectDirFromCommandline(int32 ArgC, TCHAR* ArgV[]);
}
