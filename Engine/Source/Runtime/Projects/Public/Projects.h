// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Public Dependencies
 *****************************************************************************/

#include "Core.h"


/* Public Includes
 *****************************************************************************/

#include "ModuleDescriptor.h"
#include "LocalizationDescriptor.h"
#include "CustomBuildSteps.h"
#include "PluginDescriptor.h"
#include "Interfaces/IPluginManager.h"
#include "ProjectDescriptor.h"
#include "Interfaces/IProjectManager.h"
